{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Apple ARKit Face Support",
	"Description": "Support for Apple's face tracking features",
	"Category": "Augmented Reality",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"EnabledByDefault": false,
	"SupportedPrograms": ["LiveLinkHub"],
	"SupportedTargetPlatforms" : [
		"IOS", "Win64", "Mac","Linux", "Android"
	],
	"Modules":
	[
		{
			"Name": "AppleARKitFaceSupport",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"ProgramAllowList": ["LiveLinkHub"],
			"PlatformAllowList" : [ "Mac", "IOS", "Win64" ,"Linux", "Android"]
		}
	],
    "Plugins":
    [
		{
			"Name": "AppleARKit",
			"Enabled": true
		},
		{
			"Name": "ProceduralMeshComponent",
			"Enabled": true
		},
        {
            "Name": "AppleImageUtils",
            "Enabled": true
        }
    ]
}
