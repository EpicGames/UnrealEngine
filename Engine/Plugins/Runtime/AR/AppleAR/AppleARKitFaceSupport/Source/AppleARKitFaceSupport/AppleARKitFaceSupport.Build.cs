// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AppleARKitFaceSupport : ModuleRules
{
	public AppleARKitFaceSupport(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine",
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
                "HeadMountedDisplay",
				"XRBase",
                "AugmentedReality",
                "ProceduralMeshComponent",
                "LiveLinkInterface",
//                "OnlineSubsystem",
                "Sockets",
                "AppleARKit",
                "AppleImageUtils"
				// ... add private dependencies that you statically link with here ...
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);

		if (Target.Platform == UnrealTargetPlatform.IOS)
		{
			PublicFrameworks.Add( "ARKit" );
		}
		
		if (Target.bCompileAgainstEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Slate",
					"SlateCore",
					"PropertyEditor"
				}
			);
		}
	}
}
