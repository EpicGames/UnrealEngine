// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitFaceMeshComponent.h"
#include "ARBlueprintLibrary.h"
#include "AppleARKitFaceSupportModule.h"

#if SUPPORTS_ARKIT_1_0
	#import <ARKit/ARKit.h>
#endif

#include "AppleARKitFaceMeshConversion.h"
#include "ARTrackable.h"
#include "ARSessionConfig.h"
#include "AppleARKitSettings.h"
#include "AppleARKitConversion.h"
#include "Net/UnrealNetwork.h"

DECLARE_CYCLE_STAT(TEXT("Update"), STAT_FaceAR_Component_Update, STATGROUP_FaceAR);

#if SUPPORTS_ARKIT_1_0

NSDictionary<ARBlendShapeLocation,NSNumber *>* ToBlendShapeDictionary(const FARBlendShapeMap& BlendShapeMap)
{
	NSMutableDictionary<ARBlendShapeLocation,NSNumber *>* BlendShapeDict = [[[NSMutableDictionary<ARBlendShapeLocation,NSNumber *> alloc] init] autorelease];

#define SET_BLEND_SHAPE(AppleShape, UEShape) \
	if (BlendShapeMap.Contains(UEShape)) \
	{ \
		NSNumber* Num = [NSNumber numberWithFloat: BlendShapeMap[UEShape]]; \
		BlendShapeDict[AppleShape] = Num; \
	}

	// Do we want to capture face performance or look at the face as if in a mirror (Apple is mirrored so we mirror the mirror)
	if (GetMutableDefault<UAppleARKitSettings>()->GetFaceTrackingDirection() == EARFaceTrackingDirection::FaceMirrored)
	{
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeBlinkLeft, EARFaceBlendShape::EyeBlinkLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookDownLeft, EARFaceBlendShape::EyeLookDownLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookInLeft, EARFaceBlendShape::EyeLookInLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookOutLeft, EARFaceBlendShape::EyeLookOutLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookUpLeft, EARFaceBlendShape::EyeLookUpLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeSquintLeft, EARFaceBlendShape::EyeSquintLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeWideLeft, EARFaceBlendShape::EyeWideLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeBlinkRight, EARFaceBlendShape::EyeBlinkRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookDownRight, EARFaceBlendShape::EyeLookDownRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookInRight, EARFaceBlendShape::EyeLookInRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookOutRight, EARFaceBlendShape::EyeLookOutRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookUpRight, EARFaceBlendShape::EyeLookUpRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeSquintRight, EARFaceBlendShape::EyeSquintRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeWideRight, EARFaceBlendShape::EyeWideRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationJawForward, EARFaceBlendShape::JawForward);
		SET_BLEND_SHAPE(ARBlendShapeLocationJawLeft, EARFaceBlendShape::JawLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationJawRight, EARFaceBlendShape::JawRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthLeft, EARFaceBlendShape::MouthLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthRight, EARFaceBlendShape::MouthRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthSmileLeft, EARFaceBlendShape::MouthSmileLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthSmileRight, EARFaceBlendShape::MouthSmileRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthFrownLeft, EARFaceBlendShape::MouthFrownLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthFrownRight, EARFaceBlendShape::MouthFrownRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthDimpleLeft, EARFaceBlendShape::MouthDimpleLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthDimpleRight, EARFaceBlendShape::MouthDimpleRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthStretchLeft, EARFaceBlendShape::MouthStretchLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthStretchRight, EARFaceBlendShape::MouthStretchRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthPressLeft, EARFaceBlendShape::MouthPressLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthPressRight, EARFaceBlendShape::MouthPressRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthLowerDownLeft, EARFaceBlendShape::MouthLowerDownLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthLowerDownRight, EARFaceBlendShape::MouthLowerDownRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthUpperUpLeft, EARFaceBlendShape::MouthUpperUpLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthUpperUpRight, EARFaceBlendShape::MouthUpperUpRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationBrowDownLeft, EARFaceBlendShape::BrowDownLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationBrowDownRight, EARFaceBlendShape::BrowDownRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationBrowOuterUpLeft, EARFaceBlendShape::BrowOuterUpLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationBrowOuterUpRight, EARFaceBlendShape::BrowOuterUpRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationCheekSquintLeft, EARFaceBlendShape::CheekSquintLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationCheekSquintRight, EARFaceBlendShape::CheekSquintRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationNoseSneerLeft, EARFaceBlendShape::NoseSneerLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationNoseSneerRight, EARFaceBlendShape::NoseSneerRight);
	}
	else
	{
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeBlinkLeft, EARFaceBlendShape::EyeBlinkRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookDownLeft, EARFaceBlendShape::EyeLookDownRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookInLeft, EARFaceBlendShape::EyeLookInRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookOutLeft, EARFaceBlendShape::EyeLookOutRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookUpLeft, EARFaceBlendShape::EyeLookUpRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeSquintLeft, EARFaceBlendShape::EyeSquintRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeWideLeft, EARFaceBlendShape::EyeWideRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeBlinkRight, EARFaceBlendShape::EyeBlinkLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookDownRight, EARFaceBlendShape::EyeLookDownLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookInRight, EARFaceBlendShape::EyeLookInLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookOutRight, EARFaceBlendShape::EyeLookOutLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookUpRight, EARFaceBlendShape::EyeLookUpLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeSquintRight, EARFaceBlendShape::EyeSquintLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeWideRight, EARFaceBlendShape::EyeWideLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationJawForward, EARFaceBlendShape::JawForward);
		SET_BLEND_SHAPE(ARBlendShapeLocationJawLeft, EARFaceBlendShape::JawRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationJawRight, EARFaceBlendShape::JawLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthLeft, EARFaceBlendShape::MouthRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthRight, EARFaceBlendShape::MouthLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthSmileLeft, EARFaceBlendShape::MouthSmileRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthSmileRight, EARFaceBlendShape::MouthSmileLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthFrownLeft, EARFaceBlendShape::MouthFrownRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthFrownRight, EARFaceBlendShape::MouthFrownLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthDimpleLeft, EARFaceBlendShape::MouthDimpleRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthDimpleRight, EARFaceBlendShape::MouthDimpleLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthStretchLeft, EARFaceBlendShape::MouthStretchRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthStretchRight, EARFaceBlendShape::MouthStretchLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthPressLeft, EARFaceBlendShape::MouthPressRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthPressRight, EARFaceBlendShape::MouthPressLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthLowerDownLeft, EARFaceBlendShape::MouthLowerDownRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthLowerDownRight, EARFaceBlendShape::MouthLowerDownLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthUpperUpLeft, EARFaceBlendShape::MouthUpperUpRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthUpperUpRight, EARFaceBlendShape::MouthUpperUpLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationBrowDownLeft, EARFaceBlendShape::BrowDownRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationBrowDownRight, EARFaceBlendShape::BrowDownLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationBrowOuterUpLeft, EARFaceBlendShape::BrowOuterUpRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationBrowOuterUpRight, EARFaceBlendShape::BrowOuterUpLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationCheekSquintLeft, EARFaceBlendShape::CheekSquintRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationCheekSquintRight, EARFaceBlendShape::CheekSquintLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationNoseSneerLeft, EARFaceBlendShape::NoseSneerRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationNoseSneerRight, EARFaceBlendShape::NoseSneerLeft);
	}

	// These are the same mirrored or not
	SET_BLEND_SHAPE(ARBlendShapeLocationJawOpen, EARFaceBlendShape::JawOpen);
	SET_BLEND_SHAPE(ARBlendShapeLocationMouthClose, EARFaceBlendShape::MouthClose);
	SET_BLEND_SHAPE(ARBlendShapeLocationMouthFunnel, EARFaceBlendShape::MouthFunnel);
	SET_BLEND_SHAPE(ARBlendShapeLocationMouthPucker, EARFaceBlendShape::MouthPucker);
	SET_BLEND_SHAPE(ARBlendShapeLocationMouthRollLower, EARFaceBlendShape::MouthRollLower);
	SET_BLEND_SHAPE(ARBlendShapeLocationMouthRollUpper, EARFaceBlendShape::MouthRollUpper);
	SET_BLEND_SHAPE(ARBlendShapeLocationMouthShrugLower, EARFaceBlendShape::MouthShrugLower);
	SET_BLEND_SHAPE(ARBlendShapeLocationMouthShrugUpper, EARFaceBlendShape::MouthShrugUpper);
	SET_BLEND_SHAPE(ARBlendShapeLocationBrowInnerUp, EARFaceBlendShape::BrowInnerUp);
	SET_BLEND_SHAPE(ARBlendShapeLocationCheekPuff, EARFaceBlendShape::CheekPuff);

#undef SET_BLEND_SHAPE

	return BlendShapeDict;
}

#endif

UAppleARKitFaceMeshComponent::UAppleARKitFaceMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	// Tick late in the frame to make sure the ARKit has had a chance to update
	PrimaryComponentTick.TickGroup = TG_PostPhysics;
	// Init the blend shape map so that all blend shapes are present
	for (int32 Shape = 0; Shape < (int32)EARFaceBlendShape::MAX; Shape++)
	{
		BlendShapes.Add((EARFaceBlendShape)Shape, 0.f);
	}
}

void UAppleARKitFaceMeshComponent::InitializeComponent()
{
	Super::InitializeComponent();

	if (bAutoBindToLocalFaceMesh)
	{
		PrimaryComponentTick.SetTickFunctionEnable(true);
	}
}

void UAppleARKitFaceMeshComponent::CreateMesh(const TArray<FVector>& Vertices, const TArray<int32>& Triangles, const TArray<FVector2D>& UV0)
{
	// @todo JoeG - add a counter here so we can measure the cost
	TArray<FVector> Normals;
	TArray<FLinearColor> VertexColors;
	TArray<FProcMeshTangent> Tangents;

	CreateMeshSection_LinearColor(0, Vertices, Triangles, Normals, UV0, VertexColors, Tangents, bWantsCollision);
}

void UAppleARKitFaceMeshComponent::SetBlendShapes(const TMap<EARFaceBlendShape, float>& InBlendShapes)
{
	LastUpdateFrameNumber++;
	LastUpdateTimestamp = FPlatformTime::Seconds();
	BuildUpdatedCurves(InBlendShapes);
}

void UAppleARKitFaceMeshComponent::SetBlendShapeAmount(EARFaceBlendShape BlendShape, float Amount)
{
	RemoteCurves.Reset();
	float OldVal = BlendShapes[BlendShape];
	if (FNetQuantizeFaceCurve::IsDifferentEnough(OldVal, Amount))
	{
		new(RemoteCurves) FNetQuantizeFaceCurve(BlendShape, Amount);
	}
	if (GetNetMode() == NM_Client)
	{
		// Send to the server for replication
		ServerUpdateFaceCurves(RemoteCurves);
	}
	// Merge them in using the OnRep
	OnRep_RemoteCurves();
}

float UAppleARKitFaceMeshComponent::GetFaceBlendShapeAmount(EARFaceBlendShape BlendShape) const
{
	float BlendShapeAmount = 0.f;
	if (BlendShapes.Contains(BlendShape))
	{
		BlendShapeAmount = BlendShapes[BlendShape];
	}
	return BlendShapeAmount;
}

FMatrix UAppleARKitFaceMeshComponent::GetRenderMatrix() const
{
	const float Scale = FAppleARKitConversion::ToUEScale();
	
	FTransform RenderTrans;
	switch (TransformSetting)
	{
		case EARFaceComponentTransformMixing::ComponentOnly:
		{
			RenderTrans = GetComponentTransform();
			break;
		}
		case EARFaceComponentTransformMixing::ComponentLocationTrackedRotation:
		{
			RenderTrans = GetComponentTransform();
			FQuat TrackedRot = LocalToWorldTransform.GetRotation();
			RenderTrans.SetRotation(TrackedRot);
			break;
		}
		case EARFaceComponentTransformMixing::ComponentWithTracked:
		{
			RenderTrans = GetComponentTransform() * LocalToWorldTransform;
			break;
		}
		case EARFaceComponentTransformMixing::TrackingOnly:
		{
			RenderTrans = LocalToWorldTransform;
			break;
		}
	}
	RenderTrans.MultiplyScale3D(FVector(Scale));
	return RenderTrans.ToMatrixWithScale();
}

class UMaterialInterface* UAppleARKitFaceMeshComponent::GetMaterial(int32 ElementIndex) const
{
	if (ElementIndex == 0)
	{
		return FaceMaterial;
	}
	return nullptr;
}

void UAppleARKitFaceMeshComponent::SetMaterial(int32 ElementIndex, class UMaterialInterface* Material)
{
	if (ElementIndex == 0)
	{
		FaceMaterial = Material;
	}
}

void UAppleARKitFaceMeshComponent::UpdateMeshFromBlendShapes()
{
	LocalToWorldTransform = FTransform::Identity;
#if SUPPORTS_ARKIT_1_0
	// @todo JoeG - add a counter here so we can measure the cost
	NSDictionary<ARBlendShapeLocation,NSNumber *>* BlendShapeDict = ToBlendShapeDictionary(BlendShapes);
	ARFaceGeometry* FaceGeo = [[ARFaceGeometry alloc] initWithBlendShapes: BlendShapeDict];
	if (FaceGeo != nullptr)
	{
		// Create or update the mesh depending on if we've been created before
		if (GetNumSections() > 0)
		{
			UpdateMesh(ToVertexBuffer(FaceGeo.vertices, FaceGeo.vertexCount));
		}
		else
		{
			// @todo JoeG - route the uvs in
			TArray<FVector2D> UVs;
			CreateMesh(ToVertexBuffer(FaceGeo.vertices, FaceGeo.vertexCount), To32BitIndexBuffer(FaceGeo.triangleIndices, FaceGeo.triangleCount * 3), UVs);
		}
	}
	[FaceGeo release];
#endif
}

void UAppleARKitFaceMeshComponent::UpdateMesh(const TArray<FVector>& Vertices)
{
	// @todo JoeG - add a counter here so we can measure the cost
	TArray<FVector> Normals;
	TArray<FVector2D> UV0;
	TArray<FLinearColor> VertexColors;
	TArray<FProcMeshTangent> Tangents;

	UpdateMeshSection_LinearColor(0, Vertices, Normals, UV0, VertexColors, Tangents);
}

int32 UAppleARKitFaceMeshComponent::GetLastUpdateFrameNumber() const
{
	return LastUpdateFrameNumber;
}

float UAppleARKitFaceMeshComponent::GetLastUpdateTimestamp() const
{
	return LastUpdateTimestamp;
}

UARFaceGeometry* UAppleARKitFaceMeshComponent::FindFaceGeometry()
{
	const TArray<UARTrackedGeometry*> Geometries = UARBlueprintLibrary::GetAllGeometries();
	for (UARTrackedGeometry* Geo : Geometries)
	{
		if (UARFaceGeometry* FaceGeo = Cast<UARFaceGeometry>(Geo))
		{
			return FaceGeo;
		}
	}
	return nullptr;
}

void UAppleARKitFaceMeshComponent::SetAutoBind(bool bAutoBind)
{
	if (bAutoBindToLocalFaceMesh != bAutoBind)
	{
		bAutoBindToLocalFaceMesh = bAutoBind;
		PrimaryComponentTick.SetTickFunctionEnable(bAutoBind);
	}
}

void UAppleARKitFaceMeshComponent::TickComponent(float DeltaTime, ELevelTick, FActorComponentTickFunction*)
{
	SCOPE_CYCLE_COUNTER(STAT_FaceAR_Component_Update);

	if (!bAutoBindToLocalFaceMesh)
	{
		return;
	}
	// Find the tracked face geometry and skip updates if it is not found (can happen if tracking is lost)
	if (UARFaceGeometry* FaceGeometry = FindFaceGeometry())
	{
		LocalToWorldTransform = FaceGeometry->GetLocalToWorldTransform();
		// +X is the default for Unreal, but you probably want the face point out of the screen (-X)
		if (bFlipTrackedRotation)
		{
			FRotator TrackedRot(LocalToWorldTransform.GetRotation());
			TrackedRot.Yaw = TrackedRot.Yaw - 180.f;
			TrackedRot.Pitch = -TrackedRot.Pitch;
			TrackedRot.Roll = -TrackedRot.Roll;
			LocalToWorldTransform.SetRotation(FQuat(TrackedRot));
		}

		LastUpdateFrameNumber = FaceGeometry->GetLastUpdateFrameNumber();
		LastUpdateTimestamp = FaceGeometry->GetLastUpdateTimestamp();
		BuildUpdatedCurves(FaceGeometry->GetBlendShapes());

		// Create or update the mesh depending on if we've been created before
		if (GetNumSections() > 0)
		{
			UpdateMesh(FaceGeometry->GetVertexBuffer());
		}
		else
		{
			CreateMesh(FaceGeometry->GetVertexBuffer(), FaceGeometry->GetIndexBuffer(), FaceGeometry->GetUVs());
		}
	}
}

void UAppleARKitFaceMeshComponent::PublishViaLiveLink(FName SubjectName)
{
	LiveLinkSubjectName = SubjectName;
	if (!LiveLinkSource.IsValid())
	{
		LiveLinkSource = FAppleARKitLiveLinkSourceFactory::CreateLiveLinkSource();
	}
}

FTransform UAppleARKitFaceMeshComponent::GetTransform() const
{
	return LocalToWorldTransform;
}

void UAppleARKitFaceMeshComponent::OnRep_RemoteCurves()
{
	if (RemoteCurves.Num() > 0)
	{
		// Merge the values into our map
		for (const FNetQuantizeFaceCurve& Curve : RemoteCurves)
		{
			BlendShapes.Add(Curve.GetBlendShape(), Curve.GetAmountAsFloat());
		}

		// We have to manage this since the ar system isn't updating this locally for us
		LastUpdateTimestamp = FPlatformTime::Seconds();

		// If we are publishing to LiveLink, then push the update out
		if (LiveLinkSource.IsValid())
		{
			TOptional<FQualifiedFrameTime> FrameTimeOpt = FApp::GetCurrentFrameTime();

			if (FrameTimeOpt.IsSet())
			{
				LiveLinkSource->PublishBlendShapes(LiveLinkSubjectName, *FrameTimeOpt, BlendShapes);
			}
			else
			{
				LiveLinkSource->PublishBlendShapes(LiveLinkSubjectName, FQualifiedFrameTime(), BlendShapes);
			}
		}
	}
}

void UAppleARKitFaceMeshComponent::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);
	
	DOREPLIFETIME(UAppleARKitFaceMeshComponent, RemoteCurves);
}

bool UAppleARKitFaceMeshComponent::ServerUpdateFaceCurves_Validate(const TArray<FNetQuantizeFaceCurve>& ClientCurves)
{
	return ClientCurves.Num() < (int32)EARFaceBlendShape::MAX;
}

void UAppleARKitFaceMeshComponent::ServerUpdateFaceCurves_Implementation(const TArray<FNetQuantizeFaceCurve>& ClientCurves)
{
	RemoteCurves = ClientCurves;
	// Publish these locally for the server
	OnRep_RemoteCurves();
}

void UAppleARKitFaceMeshComponent::BuildUpdatedCurves(const FARBlendShapeMap& NewCurves)
{
	RemoteCurves.Reset();
	// Build the set of deltas
	for (uint8 Shape = 0; Shape < (uint8)EARFaceBlendShape::MAX; Shape++)
	{
		EARFaceBlendShape BlendShape = (EARFaceBlendShape)Shape;
		float OldVal = BlendShapes[BlendShape];
		float NewVal = NewCurves[BlendShape];
		if (FNetQuantizeFaceCurve::IsDifferentEnough(OldVal, NewVal))
		{
			new(RemoteCurves) FNetQuantizeFaceCurve(BlendShape, NewVal);
		}
	}
	if (GetNetMode() == NM_Client)
	{
		// Send to the server for replication
		ServerUpdateFaceCurves(RemoteCurves);
	}
	// Merge them in using the OnRep
	OnRep_RemoteCurves();
}
