// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AppleARKitAvailability.h"
#include "AppleARKitSettings.h"

#if SUPPORTS_ARKIT_1_0

static FORCEINLINE TArray<int32> To32BitIndexBuffer(const int16_t* Indices, const uint64 IndexCount)
{
	check(IndexCount % 3 == 0);

	TArray<int32> IndexBuffer;
	IndexBuffer.AddUninitialized(IndexCount);
	for (uint32 Index = 0; Index < IndexCount; Index += 3)
	{
		IndexBuffer[Index] = (int32)Indices[Index];
		// We need to reverse the winding order
		IndexBuffer[Index + 1] = (int32)Indices[Index + 2];
		IndexBuffer[Index + 2] = (int32)Indices[Index + 1];
	}
	return IndexBuffer;
}

// @todo JoeG - An option for which way to orient tris (down +X or -X)
static FORCEINLINE TArray<FVector> ToVertexBuffer(const vector_float3* Vertices, const uint64 VertexCount)
{
	TArray<FVector> VertexBuffer;
	VertexBuffer.AddUninitialized(VertexCount);
	// @todo JoeG - make a fast routine for this
	for (int32 Index = 0; Index < VertexBuffer.Num(); Index++)
	{
		VertexBuffer[Index] = FVector(Vertices[Index].z, Vertices[Index].x, Vertices[Index].y);
	}
	return VertexBuffer;
}

static FORCEINLINE FARBlendShapeMap ToBlendShapeMap(bool bFaceMirrored, NSDictionary<ARBlendShapeLocation,NSNumber *>* BlendShapes, const FTransform& Transform, const FTransform& LeftEyeTransform, const FTransform& RightEyeTransform)
{
	FARBlendShapeMap BlendShapeMap;
	FRotator TrackedRot(Transform.GetRotation());
	// Map the -180..180 range to -1..1
	float HeadYaw = (float)TrackedRot.Yaw / 180.f;
	float HeadPitch = (float)TrackedRot.Pitch / 180.f;
	float HeadRoll = (float)TrackedRot.Roll / 180.f;
	if (!bFaceMirrored)
	{
		HeadYaw = -HeadYaw;
		HeadRoll = -HeadRoll;
	}
	BlendShapeMap.Add(EARFaceBlendShape::HeadYaw, HeadYaw);
	BlendShapeMap.Add(EARFaceBlendShape::HeadPitch, HeadPitch);
	BlendShapeMap.Add(EARFaceBlendShape::HeadRoll, HeadRoll);
	FRotator LeftEyeRot(LeftEyeTransform.GetRotation());
	float LeftEyeYaw = (float)LeftEyeRot.Yaw / 180.f;
	float LeftEyePitch = (float)LeftEyeRot.Pitch / 180.f;
	float LeftEyeRoll = (float)LeftEyeRot.Roll / 180.f;
	if (!bFaceMirrored)
	{
		LeftEyeYaw = -LeftEyeYaw;
		LeftEyeRoll = -LeftEyeRoll;
	}
	BlendShapeMap.Add(EARFaceBlendShape::LeftEyeYaw, LeftEyeYaw);
	BlendShapeMap.Add(EARFaceBlendShape::LeftEyePitch, LeftEyePitch);
	BlendShapeMap.Add(EARFaceBlendShape::LeftEyeRoll, LeftEyeRoll);
	FRotator RightEyeRot(RightEyeTransform.GetRotation());
	float RightEyeYaw = (float)RightEyeRot.Yaw / 180.f;
	float RightEyePitch = (float)RightEyeRot.Pitch / 180.f;
	float RightEyeRoll = (float)RightEyeRot.Roll / 180.f;
	if (!bFaceMirrored)
	{
		RightEyeYaw = -RightEyeYaw;
		RightEyeRoll = -RightEyeRoll;
	}
	BlendShapeMap.Add(EARFaceBlendShape::RightEyeYaw, RightEyeYaw);
	BlendShapeMap.Add(EARFaceBlendShape::RightEyePitch, RightEyePitch);
	BlendShapeMap.Add(EARFaceBlendShape::RightEyeRoll, RightEyeRoll);


#define SET_BLEND_SHAPE(AppleShape, UEShape) \
	if (BlendShapes[AppleShape]) \
	{ \
		BlendShapeMap.Add(UEShape, FMath::Max([BlendShapes[AppleShape] floatValue], 0.f)); \
	} \
	else \
	{ \
		BlendShapeMap.Add(UEShape, 0.f); \
	}

	if (bFaceMirrored)
	{
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeBlinkLeft, EARFaceBlendShape::EyeBlinkLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookDownLeft, EARFaceBlendShape::EyeLookDownLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookInLeft, EARFaceBlendShape::EyeLookInLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookOutLeft, EARFaceBlendShape::EyeLookOutLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookUpLeft, EARFaceBlendShape::EyeLookUpLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeSquintLeft, EARFaceBlendShape::EyeSquintLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeWideLeft, EARFaceBlendShape::EyeWideLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeBlinkRight, EARFaceBlendShape::EyeBlinkRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookDownRight, EARFaceBlendShape::EyeLookDownRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookInRight, EARFaceBlendShape::EyeLookInRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookOutRight, EARFaceBlendShape::EyeLookOutRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookUpRight, EARFaceBlendShape::EyeLookUpRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeSquintRight, EARFaceBlendShape::EyeSquintRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeWideRight, EARFaceBlendShape::EyeWideRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationJawForward, EARFaceBlendShape::JawForward);
		SET_BLEND_SHAPE(ARBlendShapeLocationJawLeft, EARFaceBlendShape::JawLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationJawRight, EARFaceBlendShape::JawRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthLeft, EARFaceBlendShape::MouthLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthRight, EARFaceBlendShape::MouthRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthSmileLeft, EARFaceBlendShape::MouthSmileLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthSmileRight, EARFaceBlendShape::MouthSmileRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthFrownLeft, EARFaceBlendShape::MouthFrownLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthFrownRight, EARFaceBlendShape::MouthFrownRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthDimpleLeft, EARFaceBlendShape::MouthDimpleLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthDimpleRight, EARFaceBlendShape::MouthDimpleRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthStretchLeft, EARFaceBlendShape::MouthStretchLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthStretchRight, EARFaceBlendShape::MouthStretchRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthPressLeft, EARFaceBlendShape::MouthPressLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthPressRight, EARFaceBlendShape::MouthPressRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthLowerDownLeft, EARFaceBlendShape::MouthLowerDownLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthLowerDownRight, EARFaceBlendShape::MouthLowerDownRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthUpperUpLeft, EARFaceBlendShape::MouthUpperUpLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthUpperUpRight, EARFaceBlendShape::MouthUpperUpRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationBrowDownLeft, EARFaceBlendShape::BrowDownLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationBrowDownRight, EARFaceBlendShape::BrowDownRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationBrowOuterUpLeft, EARFaceBlendShape::BrowOuterUpLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationBrowOuterUpRight, EARFaceBlendShape::BrowOuterUpRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationCheekSquintLeft, EARFaceBlendShape::CheekSquintLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationCheekSquintRight, EARFaceBlendShape::CheekSquintRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationNoseSneerLeft, EARFaceBlendShape::NoseSneerLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationNoseSneerRight, EARFaceBlendShape::NoseSneerRight);
	}
	else
	{
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeBlinkLeft, EARFaceBlendShape::EyeBlinkRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookDownLeft, EARFaceBlendShape::EyeLookDownRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookInLeft, EARFaceBlendShape::EyeLookInRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookOutLeft, EARFaceBlendShape::EyeLookOutRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookUpLeft, EARFaceBlendShape::EyeLookUpRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeSquintLeft, EARFaceBlendShape::EyeSquintRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeWideLeft, EARFaceBlendShape::EyeWideRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeBlinkRight, EARFaceBlendShape::EyeBlinkLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookDownRight, EARFaceBlendShape::EyeLookDownLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookInRight, EARFaceBlendShape::EyeLookInLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookOutRight, EARFaceBlendShape::EyeLookOutLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeLookUpRight, EARFaceBlendShape::EyeLookUpLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeSquintRight, EARFaceBlendShape::EyeSquintLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationEyeWideRight, EARFaceBlendShape::EyeWideLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationJawForward, EARFaceBlendShape::JawForward);
		SET_BLEND_SHAPE(ARBlendShapeLocationJawLeft, EARFaceBlendShape::JawRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationJawRight, EARFaceBlendShape::JawLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthLeft, EARFaceBlendShape::MouthRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthRight, EARFaceBlendShape::MouthLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthSmileLeft, EARFaceBlendShape::MouthSmileRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthSmileRight, EARFaceBlendShape::MouthSmileLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthFrownLeft, EARFaceBlendShape::MouthFrownRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthFrownRight, EARFaceBlendShape::MouthFrownLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthDimpleLeft, EARFaceBlendShape::MouthDimpleRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthDimpleRight, EARFaceBlendShape::MouthDimpleLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthStretchLeft, EARFaceBlendShape::MouthStretchRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthStretchRight, EARFaceBlendShape::MouthStretchLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthPressLeft, EARFaceBlendShape::MouthPressRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthPressRight, EARFaceBlendShape::MouthPressLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthLowerDownLeft, EARFaceBlendShape::MouthLowerDownRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthLowerDownRight, EARFaceBlendShape::MouthLowerDownLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthUpperUpLeft, EARFaceBlendShape::MouthUpperUpRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationMouthUpperUpRight, EARFaceBlendShape::MouthUpperUpLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationBrowDownLeft, EARFaceBlendShape::BrowDownRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationBrowDownRight, EARFaceBlendShape::BrowDownLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationBrowOuterUpLeft, EARFaceBlendShape::BrowOuterUpRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationBrowOuterUpRight, EARFaceBlendShape::BrowOuterUpLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationCheekSquintLeft, EARFaceBlendShape::CheekSquintRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationCheekSquintRight, EARFaceBlendShape::CheekSquintLeft);
		SET_BLEND_SHAPE(ARBlendShapeLocationNoseSneerLeft, EARFaceBlendShape::NoseSneerRight);
		SET_BLEND_SHAPE(ARBlendShapeLocationNoseSneerRight, EARFaceBlendShape::NoseSneerLeft);
	}

	// These are the same mirrored or not
	SET_BLEND_SHAPE(ARBlendShapeLocationJawOpen, EARFaceBlendShape::JawOpen);
	SET_BLEND_SHAPE(ARBlendShapeLocationMouthClose, EARFaceBlendShape::MouthClose);
	SET_BLEND_SHAPE(ARBlendShapeLocationMouthFunnel, EARFaceBlendShape::MouthFunnel);
	SET_BLEND_SHAPE(ARBlendShapeLocationMouthPucker, EARFaceBlendShape::MouthPucker);
	SET_BLEND_SHAPE(ARBlendShapeLocationMouthRollLower, EARFaceBlendShape::MouthRollLower);
	SET_BLEND_SHAPE(ARBlendShapeLocationMouthRollUpper, EARFaceBlendShape::MouthRollUpper);
	SET_BLEND_SHAPE(ARBlendShapeLocationMouthShrugLower, EARFaceBlendShape::MouthShrugLower);
	SET_BLEND_SHAPE(ARBlendShapeLocationMouthShrugUpper, EARFaceBlendShape::MouthShrugUpper);
	SET_BLEND_SHAPE(ARBlendShapeLocationBrowInnerUp, EARFaceBlendShape::BrowInnerUp);
	SET_BLEND_SHAPE(ARBlendShapeLocationCheekPuff, EARFaceBlendShape::CheekPuff);
#if SUPPORTS_ARKIT_2_0
	if (FAppleARKitAvailability::SupportsARKit20())
	{
		SET_BLEND_SHAPE(ARBlendShapeLocationTongueOut, EARFaceBlendShape::TongueOut);
	}
	else
	{
		// Always add a zeroed entry for any unsupported blend shapes
		BlendShapeMap.Add(EARFaceBlendShape::TongueOut, 0.f);
	}
#else
	// Always add a zeroed entry for any unsupported blend shapes
	BlendShapeMap.Add(EARFaceBlendShape::TongueOut, 0.f);
#endif

#undef SET_BLEND_SHAPE

	return BlendShapeMap;
}

#endif
