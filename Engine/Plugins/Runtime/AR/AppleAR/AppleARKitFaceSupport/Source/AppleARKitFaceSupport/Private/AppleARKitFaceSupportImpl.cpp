// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitFaceSupportImpl.h"
#include "AppleARKitSettings.h"
#include "AppleARKitFaceMeshConversion.h"
#include "AppleARKitConversion.h"
#include "Async/TaskGraphInterfaces.h"
#include "ARSystem.h"
#include "Misc/ConfigCacheIni.h"
#include "AppleARKitFaceSupportModule.h"
#include "Async/Async.h"
#include "Engine/TimecodeProvider.h"
#include "Math/UnrealMathUtility.h"

DECLARE_CYCLE_STAT(TEXT("Conversion"), STAT_FaceAR_Conversion, STATGROUP_FaceAR);

// MERGE-todo
//bool FAppleARKitFaceSupport::bNeedsInit = true;

#if SUPPORTS_ARKIT_1_0

static TSharedPtr<FAppleARKitAnchorData> MakeAnchorData(bool bFaceMirrored, ARAnchor* Anchor, const FRotator& AdjustBy, EARFaceTrackingUpdate UpdateSetting, const FTimecode& Timecode, uint32 FrameRate)
{
	SCOPE_CYCLE_COUNTER(STAT_FaceAR_Conversion);
	
    TSharedPtr<FAppleARKitAnchorData> NewAnchor;
    if ([Anchor isKindOfClass:[ARFaceAnchor class]])
    {
        ARFaceAnchor* FaceAnchor = (ARFaceAnchor*)Anchor;
		FTransform LeftEyeTransform;
		FTransform RightEyeTransform;
		FVector LookAtTarget;
#if SUPPORTS_ARKIT_2_0
		if (FAppleARKitAvailability::SupportsARKit20())
		{
			LeftEyeTransform = FAppleARKitConversion::ToFTransform(FaceAnchor.leftEyeTransform, AdjustBy);
			RightEyeTransform = FAppleARKitConversion::ToFTransform(FaceAnchor.rightEyeTransform, AdjustBy);
			LookAtTarget = FAppleARKitConversion::ToFVector(FaceAnchor.lookAtPoint);
		}
#endif
		const auto NumVertices = FaceAnchor.geometry.vertexCount;
		// https://developer.apple.com/documentation/arkit/arfacegeometry/2931118-texturecoordinates
		// The UV of the face mesh never changes so it can be cached here
		static TArray<FVector2D> FaceUVData;
		if (FaceUVData.Num() != NumVertices && FaceAnchor.geometry.textureCoordinates)
		{
			FaceUVData.Reset(NumVertices);
			for (auto Index = 0; Index < NumVertices; ++Index)
			{
				const auto& UV = FaceAnchor.geometry.textureCoordinates[Index];
				FaceUVData.Add({ UV.x, UV.y });
			}
		}
		NewAnchor = MakeShared<FAppleARKitAnchorData>(
			FAppleARKitConversion::ToFGuid(FaceAnchor.identifier),
			FAppleARKitConversion::ToFTransform(FaceAnchor.transform, AdjustBy),
			ToBlendShapeMap(bFaceMirrored, FaceAnchor.blendShapes, FAppleARKitConversion::ToFTransform(FaceAnchor.transform, AdjustBy), LeftEyeTransform, RightEyeTransform),
			UpdateSetting == EARFaceTrackingUpdate::CurvesAndGeo ? ToVertexBuffer(FaceAnchor.geometry.vertices, NumVertices) : TArray<FVector>(),
			UpdateSetting == EARFaceTrackingUpdate::CurvesAndGeo ? FaceUVData : TArray<FVector2D>(), // Note that this will generate a copy of the UV data
			LeftEyeTransform,
			RightEyeTransform,
			LookAtTarget,
			Timecode,
			FrameRate
		);
        // Only convert from 16bit to 32bit once
        if (UpdateSetting == EARFaceTrackingUpdate::CurvesAndGeo && FAppleARKitAnchorData::FaceIndices.Num() == 0)
        {
            FAppleARKitAnchorData::FaceIndices = To32BitIndexBuffer(FaceAnchor.geometry.triangleIndices, FaceAnchor.geometry.triangleCount * 3);
        }
		NewAnchor->bIsTracked = FaceAnchor.isTracked;
    }

    return NewAnchor;
}

#endif

FAppleARKitFaceSupport::FAppleARKitFaceSupport() :
	TimecodeProvider(nullptr)
{
	bFaceMirrored = false;
	// Generate our device id
	LocalDeviceId = FName(*FPlatformMisc::GetDeviceId());
}

FAppleARKitFaceSupport::~FAppleARKitFaceSupport()
{
	// Should only be called durirng shutdown
	check(IsEngineExitRequested());
}

void FAppleARKitFaceSupport::Init()
{
	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
}

void FAppleARKitFaceSupport::Shutdown()
{
	if (RemoteLiveLinkPublisher.IsValid())
	{
		RemoteLiveLinkPublisher->Shutdown();
	}
	RemoteLiveLinkPublisher = nullptr;

	if (LiveLinkFileWriter.IsValid())
	{
		LiveLinkFileWriter->Shutdown();
	}
	LiveLinkFileWriter = nullptr;

	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
}

bool FAppleARKitFaceSupport::Exec_Runtime(UWorld*, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("LiveLinkFaceAR")))
	{
		FString RemoteIp;
		if (FParse::Value(Cmd, TEXT("SendTo="), RemoteIp))
		{
			// We need to recreate the LiveLink remote publisher
			RemoteLiveLinkPublisher = nullptr;
			// Only send from iOS to desktop
#if PLATFORM_IOS
			// This will perform the sending of the data to the remote
			RemoteLiveLinkPublisher = FAppleARKitLiveLinkSourceFactory::CreateLiveLinkRemotePublisher(RemoteIp);
#endif
			return true;
		}
	}
	return false;
}

void FAppleARKitFaceSupport::InitRealtimeProviders()
{
	static bool bNeedsInit = true;
	if (bNeedsInit)
	{
		bNeedsInit = false;
#if PLATFORM_IOS
		// This will perform the sending of the data to the remote
		RemoteLiveLinkPublisher = FAppleARKitLiveLinkSourceFactory::CreateLiveLinkRemotePublisher();
		// Create the file writer if required. Will return nullptr if not configured
		LiveLinkFileWriter = FAppleARKitLiveLinkSourceFactory::CreateLiveLinkLocalFileWriter();
#endif
	}
}
#if SUPPORTS_ARKIT_1_0

ARConfiguration* FAppleARKitFaceSupport::ToARConfiguration(UARSessionConfig* SessionConfig, UTimecodeProvider* InProvider)
{
	TimecodeProvider = InProvider;

	ARFaceTrackingConfiguration* SessionConfiguration = nullptr;
	if (SessionConfig->GetSessionType() == EARSessionType::Face)
	{
		if (ARFaceTrackingConfiguration.isSupported == FALSE)
		{
			return nullptr;
		}
		SessionConfiguration = [ARFaceTrackingConfiguration new];
	}

	// Init the remote sender and file loggers if requested
	InitRealtimeProviders();
	if (LiveLinkFileWriter.IsValid())
	{
		LiveLinkFileWriter->SetTimecodeProvider(TimecodeProvider);
	}

	// Copy / convert properties
	SessionConfiguration.lightEstimationEnabled = SessionConfig->GetLightEstimationMode() != EARLightEstimationMode::None;
	SessionConfiguration.providesAudioData = NO;
	SessionConfiguration.worldAlignment = FAppleARKitConversion::ToARWorldAlignment(SessionConfig->GetWorldAlignment());

#if SUPPORTS_ARKIT_1_5
	if (FAppleARKitAvailability::SupportsARKit15())
	{
		ARVideoFormat* Format = FAppleARKitConversion::ToARVideoFormat(SessionConfig->GetDesiredVideoFormat(), ARFaceTrackingConfiguration.supportedVideoFormats, SessionConfig->ShouldUseOptimalVideoFormat());
		if (Format != nullptr)
		{
			SessionConfiguration.videoFormat = Format;
		}
	}
#endif
	
#if SUPPORTS_ARKIT_3_0
	if (FAppleARKitAvailability::SupportsARKit30())
	{
		const auto RequestedFaces = SessionConfig->GetMaxNumberOfTrackedFaces();
		const int32 SupportedFaces = [ARFaceTrackingConfiguration supportedNumberOfTrackedFaces];
		if (RequestedFaces > SupportedFaces)
		{
			UE_LOGF(LogAppleARKitFace, Warning, "Request to support %d tracked faces but the device only supports %d!", RequestedFaces, SupportedFaces);
		}
		SessionConfiguration.maximumNumberOfTrackedFaces = FMath::Min(RequestedFaces, SupportedFaces);
	}
#endif
	
	// Do we want to capture face performance or look at the face as if in a mirror (Apple is mirrored so we mirror the mirror)
	bFaceMirrored = SessionConfig->GetFaceTrackingDirection() == EARFaceTrackingDirection::FaceMirrored;

	return SessionConfiguration;
}

TArray<TSharedPtr<FAppleARKitAnchorData>> FAppleARKitFaceSupport::MakeAnchorData(NSArray<ARAnchor*>* Anchors, const FRotator& AdjustBy, EARFaceTrackingUpdate UpdateSetting)
{
	TArray<TSharedPtr<FAppleARKitAnchorData>> AnchorList;

	const FTimecode& Timecode = TimecodeProvider->GetTimecode();
	const FFrameRate& FrameRate = TimecodeProvider->GetFrameRate();

	int32 Rate = 1;

	if (FrameRate.IsValid()) // avoids divide by zero when calling FrameRate.AsDecimal()
	{
		Rate = int32(FMath::RoundHalfFromZero(FrameRate.AsDecimal()));

		// Just in case it rounded to zero (or negative).
		if (Rate < 1)
		{
			Rate = 1;
		}
	}

	for (ARAnchor* Anchor in Anchors)
	{
		TSharedPtr<FAppleARKitAnchorData> AnchorData = ::MakeAnchorData(bFaceMirrored, Anchor, AdjustBy, UpdateSetting, Timecode, uint32(Rate));
		if (AnchorData.IsValid())
		{
			AnchorList.Add(AnchorData);
			// Process any providers that want real time access to the face curve data
			ProcessRealTimePublishers(AnchorData);
		}
	}

	return AnchorList;
}

FName FAppleARKitFaceSupport::GetLiveLinkSubjectName(const FGuid& AnchorId)
{
	// This function is called from both the game and the delegate thread
	FScopeLock Lock(&AnchorIdLock);
	if (auto Record = AnchorIdToSubjectName.Find(AnchorId))
	{
		return *Record;
	}
	
	const auto DefaultName = GetMutableDefault<UAppleARKitSettings>()->GetFaceTrackingLiveLinkSubjectName();
	if (AnchorIdToSubjectName.Num())
	{
		auto NewName = FString::Printf(TEXT("%s-%d"), *DefaultName.ToString(), AnchorIdToSubjectName.Num());
		AnchorIdToSubjectName.Add(AnchorId, *NewName);
	}
	else
	{
		AnchorIdToSubjectName.Add(AnchorId, DefaultName);
	}
	return AnchorIdToSubjectName[AnchorId];
}

void FAppleARKitFaceSupport::ProcessRealTimePublishers(TSharedPtr<FAppleARKitAnchorData> AnchorData)
{
	// Copy the data from the passed in anchor
    TSharedPtr<FAppleARKitAnchorData> AsyncAnchorCopy = MakeShared<FAppleARKitAnchorData>(*AnchorData);
	const auto SubjectName = GetLiveLinkSubjectName(AnchorData->AnchorGUID);
	AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [this, AsyncAnchorCopy, SubjectName]()
	{
		const FQualifiedFrameTime FrameTime(AsyncAnchorCopy->Timecode, FFrameRate(AsyncAnchorCopy->FrameRate, 1));
		const FARBlendShapeMap& BlendShapes = AsyncAnchorCopy->BlendShapes;

		if (RemoteLiveLinkPublisher.IsValid())
		{
			RemoteLiveLinkPublisher->PublishBlendShapes(SubjectName, FrameTime, BlendShapes, LocalDeviceId);
		}

		if (LiveLinkFileWriter.IsValid())
		{
			LiveLinkFileWriter->PublishBlendShapes(SubjectName, FrameTime, BlendShapes, LocalDeviceId);
		}
	});
}

void FAppleARKitFaceSupport::PublishLiveLinkData(const FGuid& SessionGuid, TSharedPtr<FAppleARKitAnchorData> Anchor)
{
	if (LastSessionId != SessionGuid)
	{
		// Clear the mapping if we're in a new session
		LastSessionId = SessionGuid;
		AnchorIdToSubjectName = {};
	}
	
	static bool bNeedsInit = true;
	if (bNeedsInit)
	{
		bNeedsInit = false;
	    // Create our LiveLink provider if the project setting is enabled
		if (GetMutableDefault<UAppleARKitSettings>()->IsLiveLinkEnabledForFaceTracking())
		{
#if PLATFORM_IOS
			LiveLinkSource = FAppleARKitLiveLinkSourceFactory::CreateLiveLinkSource();
#else
			// This will initialize the remote listener.
			LiveLinkSource = FAppleARKitLiveLinkSourceFactory::CreateLiveLinkSource(FAppleARKitLiveLinkConnectionSettings());
#endif
		}
	}
	
#if PLATFORM_IOS
	if (LiveLinkSource.IsValid())
	{
		const FQualifiedFrameTime FrameTime(Anchor->Timecode, FFrameRate(Anchor->FrameRate, 1));
		LiveLinkSource->PublishBlendShapes(GetLiveLinkSubjectName(Anchor->AnchorGUID), FrameTime, Anchor->BlendShapes, LocalDeviceId);
	}
#endif
}

bool FAppleARKitFaceSupport::DoesSupportFaceAR()
{
	return ARFaceTrackingConfiguration.isSupported == TRUE;
}
#endif
#if SUPPORTS_ARKIT_1_5
NSArray<ARVideoFormat*>* FAppleARKitFaceSupport::GetSupportedVideoFormats() const
{
	if (FAppleARKitAvailability::SupportsARKit15())
	{
		return ARFaceTrackingConfiguration.supportedVideoFormats;
	}
	
	return nullptr;
}
#endif

#if SUPPORTS_ARKIT_3_0
bool FAppleARKitFaceSupport::IsARFrameSemanticsSupported(ARFrameSemantics InSemantics) const
{
	if (FAppleARKitAvailability::SupportsARKit30())
	{
		return [ARFaceTrackingConfiguration supportsFrameSemantics: InSemantics];
	}
	return false;
}
#endif

int32 FAppleARKitFaceSupport::GetNumberOfTrackedFacesSupported() const
{
#if SUPPORTS_ARKIT_3_0
	if (FAppleARKitAvailability::SupportsARKit30())
	{
		return [ARFaceTrackingConfiguration supportedNumberOfTrackedFaces];
	}
#endif
	return 1;
}
