// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AppleARKitFaceSupport.h"
#include "AppleARKitLiveLinkSourceFactory.h"
#include "AppleARKitConversion.h"
#include "Misc/Guid.h"
#include "HAL/CriticalSection.h"


class APPLEARKITFACESUPPORT_API FAppleARKitFaceSupport :
	public IAppleARKitFaceSupport,
	public FSelfRegisteringExec,
	public TSharedFromThis<FAppleARKitFaceSupport, ESPMode::ThreadSafe>
{
public:
	FAppleARKitFaceSupport();
	virtual ~FAppleARKitFaceSupport();

	void Init();
	void Shutdown();

private:
#if SUPPORTS_ARKIT_1_0
	// ~IAppleARKitFaceSupport
	virtual TArray<TSharedPtr<FAppleARKitAnchorData>> MakeAnchorData(NSArray<ARAnchor*>* NewAnchors, const FRotator& AdjustBy, EARFaceTrackingUpdate UpdateSetting) override;
	virtual ARConfiguration* ToARConfiguration(UARSessionConfig* InSessionConfig, UTimecodeProvider* InProvider) override;

	virtual bool DoesSupportFaceAR() override;

#if SUPPORTS_ARKIT_1_5
	virtual NSArray<ARVideoFormat*>* GetSupportedVideoFormats() const override;
#endif

#if SUPPORTS_ARKIT_3_0
	virtual bool IsARFrameSemanticsSupported(ARFrameSemantics InSemantics) const override;
#endif
	// ~IAppleARKitFaceSupport

	/** Publishes the remote publisher and the file writer if present */
	void ProcessRealTimePublishers(TSharedPtr<FAppleARKitAnchorData> AnchorData);

	virtual void PublishLiveLinkData(const FGuid& SessionGuid, TSharedPtr<FAppleARKitAnchorData> Anchor) override;
#endif // SUPPORTS_ARKIT_1_0
	
	virtual int32 GetNumberOfTrackedFacesSupported() const override;
	
	/** Inits the real time providers if needed */
	void InitRealtimeProviders();
	
	FName GetLiveLinkSubjectName(const FGuid& AnchorId);

	/** Whether the face data is mirrored or not */
	bool bFaceMirrored;
	
protected:
	//~ FSelfRegisteringExec
	virtual bool Exec_Runtime(UWorld*, const TCHAR* Cmd, FOutputDevice& Ar) override;
	//~ FSelfRegisteringExec

	/** If requested, publishes face ar updates to LiveLink for the animation system to use */
	TSharedPtr<ILiveLinkSourceARKit> LiveLinkSource;
	/** The id of this device */
	FName LocalDeviceId;
	/** A publisher that sends to a remote machine */
	TSharedPtr<IARKitBlendShapePublisher, ESPMode::ThreadSafe> RemoteLiveLinkPublisher;
	/** A publisher that writes the data to disk */
	TSharedPtr<IARKitBlendShapePublisher, ESPMode::ThreadSafe> LiveLinkFileWriter;
	/**
	 * The time code provider to use when tagging time stamps
	 * Note: this requires the FAppleARKitSystem object to mark it in use so GC doesn't destroy it. Normally it would
	 * implement the FGCObject interface but this gets created before UObjects are init-ed so not possible
	 * */
	UTimecodeProvider* TimecodeProvider;
	
	// The id of the last session
	FGuid LastSessionId;
	
	// { Anchor Id : livelink subject name }
	TMap<FGuid, FName> AnchorIdToSubjectName;
	
	FCriticalSection AnchorIdLock;
};
