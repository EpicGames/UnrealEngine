// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitFaceSupportModule.h"

#include "AppleARKitFaceSupportImpl.h"
#include "AppleARKitLiveLinkSource.h"
#include "CoreGlobals.h"
#include "Features/IModularFeature.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"


IMPLEMENT_MODULE(FAppleARKitFaceSupportModule, AppleARKitFaceSupport);

DEFINE_LOG_CATEGORY(LogAppleARKitFace);

void FAppleARKitFaceSupportModule::StartupModule()
{
	ensureMsgf(FModuleManager::Get().LoadModule("AppleARKit"), TEXT("ARKitFaceSupport depends on the AppleARKit module."));

	FaceSupportInstance = MakeShared<FAppleARKitFaceSupport, ESPMode::ThreadSafe>();
	FaceSupportInstance->Init();

#if PLATFORM_DESKTOP || PLATFORM_ANDROID
	IModularFeatures::Get().OnModularFeatureRegistered().AddRaw(this, &FAppleARKitFaceSupportModule::OnModularFeatureAvailable);
#endif
}

void FAppleARKitFaceSupportModule::ShutdownModule()
{
	LiveLinkSource.Reset();
	FaceSupportInstance->Shutdown();
	FaceSupportInstance = nullptr;
}

void FAppleARKitFaceSupportModule::OnModularFeatureAvailable(const FName& Type, class IModularFeature* ModularFeature)
{
	// LiveLink listener needs to be created here so that we can receive remote publishing events
	if (Type == ILiveLinkClient::ModularFeatureName)
	{
		// Reset in case the modular feature was re-registered (ie. LiveLinkHub).
		LiveLinkSource.Reset();

		bool bEnableLiveLinkForFaceTracking = false;
		FAppleARKitLiveLinkConnectionSettings Settings;
		GConfig->GetBool(TEXT("/Script/AppleARKit.AppleARKitSettings"), TEXT("bEnableLiveLinkForFaceTracking"), bEnableLiveLinkForFaceTracking, GEngineIni);
		GConfig->GetInt(TEXT("/Script/AppleARKit.AppleARKitSettings"), TEXT("LiveLinkPublishingPort"), Settings.Port, GEngineIni);

		if (bEnableLiveLinkForFaceTracking || FParse::Param(FCommandLine::Get(), TEXT("EnableLiveLinkFaceTracking")))
		{
			LiveLinkSource = FAppleARKitLiveLinkSourceFactory::CreateLiveLinkSource(Settings);
		}
	}
}



