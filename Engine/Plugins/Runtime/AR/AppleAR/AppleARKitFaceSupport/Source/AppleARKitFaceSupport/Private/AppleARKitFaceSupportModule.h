// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"
#include "Modules/ModuleManager.h"

class APPLEARKITFACESUPPORT_API FAppleARKitFaceSupportModule :
	public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

private:
	/** Handler called when a modular feature is made available. Used to start a remote listener after the livelink client is registered. */
	void OnModularFeatureAvailable(const FName& Type, class IModularFeature* ModularFeature);
private:
	// Interface / helper for sending AR Face data to LiveLink from an IOS device.
	TSharedPtr<class FAppleARKitFaceSupport, ESPMode::ThreadSafe> FaceSupportInstance;

	// LiveLinkSource responsible for receiving remote LiveLink data and publishing it to the LiveLink client.
	TSharedPtr<class ILiveLinkSourceARKit> LiveLinkSource;
};

DECLARE_LOG_CATEGORY_EXTERN(LogAppleARKitFace, Log, All);

DECLARE_STATS_GROUP(TEXT("Face AR"), STATGROUP_FaceAR, STATCAT_Advanced);
