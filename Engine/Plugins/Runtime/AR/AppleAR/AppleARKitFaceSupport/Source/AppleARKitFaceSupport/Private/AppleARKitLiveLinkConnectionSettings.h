// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AppleARKitLiveLinkConnectionSettings.generated.h"

USTRUCT()
struct APPLEARKITFACESUPPORT_API FAppleARKitLiveLinkConnectionSettings
{
	GENERATED_BODY()

	/** The port to use when listening/sending LiveLink face blend shapes via the network */
	UPROPERTY(EditAnywhere, Category = "Connection Settings", meta = (ClampMin = "1", ClampMax = "32765"))
	int32 Port = 11111;
};
