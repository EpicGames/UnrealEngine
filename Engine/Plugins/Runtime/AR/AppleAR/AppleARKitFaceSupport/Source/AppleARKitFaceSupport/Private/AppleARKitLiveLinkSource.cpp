// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitLiveLinkSource.h"

#include "UObject/Package.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/ObjectMacros.h"
#include "ARSystem.h"
#include "ARBlueprintLibrary.h"
#include "Features/IModularFeatures.h"
#include "Misc/FileHelper.h"

#include "SocketSubsystem.h"
#include "IPAddress.h"
#include "Sockets.h"

#include "AppleARKitSettings.h"
#include "AppleARKitLiveLinkConnectionSettings.h"
#include "ARTrackable.h"

#include "ILiveLinkClient.h"

#include "Animation/PoseAsset.h"

#include "AppleARKitFaceSupportModule.h"

#include "Roles/LiveLinkBasicRole.h"

DECLARE_CYCLE_STAT(TEXT("Publish Local LiveLink"), STAT_FaceAR_Local_PublishLiveLink, STATGROUP_FaceAR);
DECLARE_CYCLE_STAT(TEXT("Publish Remote LiveLink"), STAT_FaceAR_Remote_PublishLiveLink, STATGROUP_FaceAR);
DECLARE_CYCLE_STAT(TEXT("Receive LiveLink"), STAT_FaceAR_ReceiveLiveLink, STATGROUP_FaceAR);

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Total LiveLink Bytes Sent"), STAT_FaceAR_Total_LiveLink_BytesSent, STATGROUP_FaceAR);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Total LiveLink Packets Sent"), STAT_FaceAR_Total_LiveLink_PacketsSent, STATGROUP_FaceAR);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Total LiveLink Bytes Recv"), STAT_FaceAR_Total_LiveLink_BytesRecv, STATGROUP_FaceAR);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Total LiveLink Packets Recv"), STAT_FaceAR_Total_LiveLink_PacketsRecv, STATGROUP_FaceAR);

DECLARE_DWORD_COUNTER_STAT(TEXT("LiveLink Bytes Sent"), STAT_FaceAR_LiveLink_BytesSent, STATGROUP_FaceAR);
DECLARE_DWORD_COUNTER_STAT(TEXT("LiveLink Packets Sent"), STAT_FaceAR_LiveLink_PacketsSent, STATGROUP_FaceAR);
DECLARE_DWORD_COUNTER_STAT(TEXT("LiveLink Bytes Recv"), STAT_FaceAR_LiveLink_BytesRecv, STATGROUP_FaceAR);
DECLARE_DWORD_COUNTER_STAT(TEXT("LiveLink Packets Recv"), STAT_FaceAR_LiveLink_PacketsRecv, STATGROUP_FaceAR);

TSharedPtr<ILiveLinkSourceARKit> FAppleARKitLiveLinkSourceFactory::CreateLiveLinkSource()
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();

	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient* LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		TSharedPtr <ILiveLinkSourceARKit> Source = MakeShareable(new FAppleARKitLiveLinkSource());
		LiveLinkClient->AddSource(Source);
		return Source;
	}
	return TSharedPtr<ILiveLinkSourceARKit>();
}

TSharedPtr<ILiveLinkSourceARKit> FAppleARKitLiveLinkSourceFactory::CreateLiveLinkSource(const FAppleARKitLiveLinkConnectionSettings& ConnectionSettings)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();

	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkClient::ModularFeatureName))
	{
		ILiveLinkClient* LiveLinkClient = &IModularFeatures::Get().GetModularFeature<ILiveLinkClient>(ILiveLinkClient::ModularFeatureName);
		TSharedPtr<FAppleARKitLiveLinkSource> Source = MakeShared<FAppleARKitLiveLinkSource>(ConnectionSettings);
		Source->InitializeRemoteListener();
		
		LiveLinkClient->AddSource(Source);

		return Source;
	}
	return TSharedPtr<ILiveLinkSourceARKit>();
}

TSharedPtr<IARKitBlendShapePublisher, ESPMode::ThreadSafe> FAppleARKitLiveLinkSourceFactory::CreateLiveLinkRemotePublisher(const FString& RemoteAddr)
{
	TSharedPtr<IARKitBlendShapePublisher, ESPMode::ThreadSafe> RemoteLiveLinkPublisher;
	// Only send from iOS to desktop
#if PLATFORM_IOS
	// This will perform the sending of the data to the remote
	FAppleARKitLiveLinkRemotePublisher* Publisher = new FAppleARKitLiveLinkRemotePublisher(RemoteAddr);
	if (Publisher->InitSendSocket())
	{
		RemoteLiveLinkPublisher = MakeShareable(Publisher);
	}
	else
	{
		UE_LOGF(LogAppleARKitFace, Warning, "Failed to create LiveLink remote publisher, so no data will be sent out");
		delete Publisher;
	}
#endif
	return RemoteLiveLinkPublisher;
}

TSharedPtr<IARKitBlendShapePublisher, ESPMode::ThreadSafe> FAppleARKitLiveLinkSourceFactory::CreateLiveLinkLocalFileWriter()
{
	TSharedPtr<IARKitBlendShapePublisher, ESPMode::ThreadSafe> LocalFileWriter;
	FString FileWriterType(TEXT("None"));
	GConfig->GetString(TEXT("/Script/AppleARKit.AppleARKitSettings"), TEXT("FaceTrackingFileWriterType"), FileWriterType, GEngineIni);
	if (FileWriterType != TEXT("None"))
	{
		FAppleARKitLiveLinkFileWriter* FileWriter = nullptr;
		if (FileWriterType == TEXT("CSV"))
		{
			FileWriter = new FAppleARKitLiveLinkFileWriterCsv();
		}
		else
		{
			FileWriter = new FAppleARKitLiveLinkFileWriterJson();
		}
		LocalFileWriter = MakeShareable(FileWriter);
	}
	return LocalFileWriter;
}

FAppleARKitLiveLinkSource::FAppleARKitLiveLinkSource() 
	: Client(nullptr)
	, LastFramePublished(0)
{
}

FAppleARKitLiveLinkSource::FAppleARKitLiveLinkSource(FAppleARKitLiveLinkConnectionSettings InConnectionSettings)
	: Client(nullptr)
	, LastFramePublished(0)
	, ConnectionSettings(MoveTemp(InConnectionSettings))
{
}

void FAppleARKitLiveLinkSource::InitializeRemoteListener()
{
	// LiveLink listener needs to be created here so that we can receive remote publishing events
#if PLATFORM_DESKTOP || PLATFORM_ANDROID
	RemoteListener = MakeUnique<FAppleARKitLiveLinkRemoteListener>(AsShared());
	if (!RemoteListener->InitReceiveSocket(ConnectionSettings.Port))
	{
		RemoteListener.Reset();
		UE_LOGF(LogAppleARKitFace, Warning, "Could not start AppleARKit remote listener on port %d", ConnectionSettings.Port);
	}
#endif
}

void FAppleARKitLiveLinkSource::ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid)
{
	Client = InClient;
	SourceGuid = InSourceGuid;
}

void FAppleARKitLiveLinkSource::InitializeSettings(ULiveLinkSourceSettings* Settings)
{
	FAppleARKitLiveLinkConnectionSettings::StaticStruct()->ExportText(Settings->ConnectionString, &ConnectionSettings, nullptr, nullptr, PPF_None, nullptr);
	Settings->Factory = UAppleARKitLiveLinkSourceFactory::StaticClass();
}

bool FAppleARKitLiveLinkSource::IsSourceStillValid() const
{
	return Client != nullptr;
}

bool FAppleARKitLiveLinkSource::RequestSourceShutdown()
{
	Client = nullptr;
	RemoteListener.Reset();
	return true;
}

FText FAppleARKitLiveLinkSource::GetSourceMachineName() const
{
	return FText().FromString(FPlatformProcess::ComputerName());
}

FText FAppleARKitLiveLinkSource::GetSourceStatus() const
{
	return NSLOCTEXT( "AppleARKitLiveLink", "AppleARKitLiveLinkStatus", "Active" );
}

FText FAppleARKitLiveLinkSource::GetSourceType() const
{
	FNumberFormattingOptions Options;
	Options.UseGrouping = false;

	return FText::Format(NSLOCTEXT("AppleARKitLiveLink", "AppleARKitLiveLinkSourceType", "ARKit Port: {0}"), FText::AsNumber(ConnectionSettings.Port, &Options));
}

static FName ParseEnumName(FName EnumName)
{
	const int32 BlendShapeEnumNameLength = 19;
	FString EnumString = EnumName.ToString();
	return FName(*EnumString.Right(EnumString.Len() - BlendShapeEnumNameLength));
}

void FAppleARKitLiveLinkSource::PublishBlendShapes(FName SubjectName, const FQualifiedFrameTime& FrameTime, const FARBlendShapeMap& FaceBlendShapes, FName DeviceId)
{
	SCOPE_CYCLE_COUNTER(STAT_FaceAR_Local_PublishLiveLink);

	check(Client != nullptr);
	
	// This code touches UObjects so needs to be run only in the game thread
	check(IsInGameThread());

	//If we can't retrieve blend shape enum, nothing we can do
	const UEnum* EnumPtr = StaticEnum<EARFaceBlendShape>();
	if (EnumPtr == nullptr)
	{
		return;
	}

	const FLiveLinkSubjectKey SubjectKey(SourceGuid, SubjectName);


	// Is this a new device and subject pair?
	FBlendShapeStaticData* BlendShapeDataPtr = BlendShapePerDeviceMap.Find(DeviceId);
	if (BlendShapeDataPtr == nullptr)
	{
		UpdateStaticData(SubjectName, FaceBlendShapes, DeviceId);
	}
	// Did the subject name change for the device?
	else if (SubjectKey != BlendShapeDataPtr->SubjectKey)
	{
		// The remote device changed subject names, so remove the old subject
		Client->RemoveSubject_AnyThread(BlendShapeDataPtr->SubjectKey);

		UpdateStaticData(SubjectName, FaceBlendShapes, DeviceId);
	}

	FLiveLinkFrameDataStruct FrameDataStruct(FLiveLinkBaseFrameData::StaticStruct());
	FLiveLinkBaseFrameData* FrameData = FrameDataStruct.Cast<FLiveLinkBaseFrameData>();
	FrameData->WorldTime = FPlatformTime::Seconds();
	FrameData->MetaData.SceneTime = FrameTime;
	FrameData->PropertyValues.Reserve((int32)EARFaceBlendShape::MAX);
	
	// Iterate through all of the blend shapes copying them into the LiveLink data type
	for (int32 Shape = 0; Shape < (int32)EARFaceBlendShape::MAX; Shape++)
	{
		if (FaceBlendShapes.Contains((EARFaceBlendShape)Shape))
		{	
			const float CurveValue = FaceBlendShapes.FindChecked((EARFaceBlendShape)Shape);
			FrameData->PropertyValues.Add(CurveValue);
		}
	}

	//Blendshapes don't change over time. If they were to change, we would need to keep track
	//of previous values to always have frame data matching static data.
	check(FrameData->PropertyValues.Num() == BlendShapePerDeviceMap.FindChecked(DeviceId).StaticData.PropertyNames.Num());
	
	// Share the data locally with the LiveLink client
	Client->PushSubjectFrameData_AnyThread(SubjectKey, MoveTemp(FrameDataStruct));
}


void FAppleARKitLiveLinkSource::UpdateStaticData(FName SubjectName, const FARBlendShapeMap& FaceBlendShapes, FName DeviceId /*= NAME_None*/)
{
	//Update the subject key to match latest one
	FBlendShapeStaticData& BlendShapeData = BlendShapePerDeviceMap.FindOrAdd(DeviceId);
	BlendShapeData.SubjectKey = FLiveLinkSubjectKey(SourceGuid, SubjectName);

	//Update property names array
	BlendShapeData.StaticData.PropertyNames.Reset((int32)EARFaceBlendShape::MAX);

	//Iterate through all valid blend shapes to extract names
	const UEnum* EnumPtr = StaticEnum<EARFaceBlendShape>();
	for (int32 Shape = 0; Shape < (int32)EARFaceBlendShape::MAX; Shape++)
	{
		if (FaceBlendShapes.Contains((EARFaceBlendShape)Shape))
		{
			//Blendshapes don't change over time. If they were to change, we would need to keep track
			//of previous values to always have frame data matching static data
			const FName ShapeName = ParseEnumName(EnumPtr->GetNameByValue(Shape));
			BlendShapeData.StaticData.PropertyNames.Add(ShapeName);
		}
	}

	//Push the associated static data
	FLiveLinkStaticDataStruct StaticDataStruct(FLiveLinkBaseStaticData::StaticStruct());
	FLiveLinkBaseStaticData* BaseStaticData = StaticDataStruct.Cast<FLiveLinkBaseStaticData>();
	BaseStaticData->PropertyNames = BlendShapeData.StaticData.PropertyNames;

	UE_LOGF(LogAppleARKitFace, Verbose, "Pushing AppleARKit Subject '%ls' with %d blend shapes", *SubjectName.ToString(), BaseStaticData->PropertyNames.Num());
	Client->PushSubjectStaticData_AnyThread(BlendShapeData.SubjectKey, ULiveLinkBasicRole::StaticClass(), MoveTemp(StaticDataStruct));
}

// 1 = Initial version
// 2 = ARKit 2.0 extra blendshapes
// 3 = Removed the timestamp to derive locally
// 4 = Added the device id to stream so we can tell devices apart
// 5 = Added timecode tracking of the curve data for synchronizing across multiple devices, media source, etc.
// 6 = Replaced FTimecode with FQualifiedFrameTime for accurate timestamping of ARKit capture, which is currently asynchronous.
const uint8 BLEND_SHAPE_PACKET_VER = 6;

//                                                                          FrameTime                     BlendShapeCount Blendshapes                                        SubjectName             DeviceID
const uint32 MAX_BLEND_SHAPE_PACKET_SIZE = sizeof(BLEND_SHAPE_PACKET_VER) + sizeof(FQualifiedFrameTime) + sizeof(uint8) + (sizeof(float) * (uint64)EARFaceBlendShape::MAX) + (sizeof(TCHAR) * 256) + (sizeof(TCHAR) * 256);
const uint32 MIN_BLEND_SHAPE_PACKET_SIZE = sizeof(BLEND_SHAPE_PACKET_VER) + sizeof(FQualifiedFrameTime) + sizeof(uint8) + (sizeof(float) * (uint64)EARFaceBlendShape::MAX) +  sizeof(TCHAR)        +  sizeof(TCHAR);

FAppleARKitLiveLinkRemotePublisher::FAppleARKitLiveLinkRemotePublisher(const FString& InRemoteIp) :
	RemoteIp(InRemoteIp),
	SendSocket(nullptr),
	SendBuffer(MAX_BLEND_SHAPE_PACKET_SIZE)
{
}

FAppleARKitLiveLinkRemotePublisher::FAppleARKitLiveLinkRemotePublisher() :
	SendSocket(nullptr),
	SendBuffer(MAX_BLEND_SHAPE_PACKET_SIZE)
{
}

FAppleARKitLiveLinkRemotePublisher::~FAppleARKitLiveLinkRemotePublisher()
{
	if (SendSocket != nullptr)
	{
		SendSocket->Close();
		ISocketSubsystem* SocketSub = ISocketSubsystem::Get();
		SocketSub->DestroySocket(SendSocket);
	}
}

bool FAppleARKitLiveLinkRemotePublisher::InitSendSocket()
{
	TSharedRef<FInternetAddr> Addr = GetSendAddress();
	if (Addr->IsValid())
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
		// Allocate our socket for sending
		SendSocket = SocketSubsystem->CreateSocket(NAME_DGram, TEXT("FAppleARKitLiveLinkRemotePublisher socket"), Addr->GetProtocolType());
		SendSocket->SetReuseAddr();
		SendSocket->SetNonBlocking();
		UE_LOGF(LogAppleARKitFace, Log, "Sending LiveLink face AR data to address (%ls)", *Addr->ToString(true));
	}
	return SendSocket != nullptr;
}

TSharedRef<FInternetAddr> FAppleARKitLiveLinkRemotePublisher::GetSendAddress()
{
	ISocketSubsystem* SocketSub = ISocketSubsystem::Get();
	TSharedRef<FInternetAddr> SendAddr = SocketSub->CreateInternetAddr();
	// If we didn't get one passed into the constructor then check the commandline for the IP
	if (!RemoteIp.Len())
	{
		FParse::Value(FCommandLine::Get(), TEXT("LiveLinkRemoteIp="), RemoteIp);
	}
	// Don't bother trying to parse the IP if it isn't set
	if (RemoteIp.Len())
	{
		int32 LiveLinkPort = GetMutableDefault<UAppleARKitSettings>()->GetLiveLinkPublishingPort();
		SendAddr->SetPort(LiveLinkPort);
		bool bIsValid = false;
		SendAddr->SetIp(*RemoteIp, bIsValid);
	}
	return SendAddr;
}

void FAppleARKitLiveLinkRemotePublisher::PublishBlendShapes(FName SubjectName, const FQualifiedFrameTime& FrameTime, const FARBlendShapeMap& FaceBlendShapes, FName DeviceId)
{
	if (SendSocket != nullptr)
	{
		SCOPE_CYCLE_COUNTER(STAT_FaceAR_Remote_PublishLiveLink);
		// Build the packet and send it
		SendBuffer.Reset();
		SendBuffer << BLEND_SHAPE_PACKET_VER;
		SendBuffer << DeviceId;
		SendBuffer << SubjectName;
		SendBuffer << FrameTime;
		uint8 BlendShapeCount = (uint8)EARFaceBlendShape::MAX;
		check(FaceBlendShapes.Num() == BlendShapeCount);
		SendBuffer << BlendShapeCount;
		// Loop through and send each float for each enum
		for (uint8 BlendShapeIndex = 0; BlendShapeIndex < BlendShapeCount; BlendShapeIndex++)
		{
			SendBuffer << FaceBlendShapes.FindChecked((EARFaceBlendShape)BlendShapeIndex);
		}

		// Now send the packet
		uint32 SourceBufferSize = SendBuffer.GetByteCount();
		check(SourceBufferSize <= MAX_BLEND_SHAPE_PACKET_SIZE && "Max packet size for face blends was exceeded");
		int32 AmountSent = 0;
		if (!SendSocket->SendTo(SendBuffer, SourceBufferSize, AmountSent, *GetSendAddress()) ||
			AmountSent != SourceBufferSize)
		{
			ISocketSubsystem* SocketSub = ISocketSubsystem::Get();
			UE_LOGF(LogAppleARKitFace, Verbose, "Failed to send face AR packet with error (%ls). Packet size (%d), sent (%d)", SocketSub->GetSocketError(), SourceBufferSize, AmountSent);
		}
		INC_DWORD_STAT(STAT_FaceAR_LiveLink_PacketsSent);
		INC_DWORD_STAT(STAT_FaceAR_Total_LiveLink_PacketsSent);
		INC_DWORD_STAT_BY(STAT_FaceAR_LiveLink_BytesSent, AmountSent);
		INC_DWORD_STAT_BY(STAT_FaceAR_Total_LiveLink_BytesSent, AmountSent);
	}
}

FAppleARKitLiveLinkRemoteListener::FAppleARKitLiveLinkRemoteListener(const TSharedPtr<ILiveLinkSourceARKit>& ArkitSource)
	: RecvSocket(nullptr)
	, Source(ArkitSource)
{
	RecvBuffer.AddUninitialized(MAX_BLEND_SHAPE_PACKET_SIZE);
}

FAppleARKitLiveLinkRemoteListener::~FAppleARKitLiveLinkRemoteListener()
{
	if (RecvSocket != nullptr)
	{
		RecvSocket->Close();

		if (!IsEngineExitRequested())
		{
			ISocketSubsystem* SocketSub = ISocketSubsystem::Get();
			SocketSub->DestroySocket(RecvSocket);
		}
	}
}

bool FAppleARKitLiveLinkRemoteListener::InitReceiveSocket(int32 Port)
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
	TSharedRef<FInternetAddr> Addr = SocketSubsystem->GetLocalBindAddr(*GLog);

	Addr->SetPort(Port);

	RecvSocket = SocketSubsystem->CreateSocket(NAME_DGram, TEXT("FAppleARKitLiveLinkRemoteListener socket"), Addr->GetProtocolType());
	if (RecvSocket != nullptr)
	{
		RecvSocket->SetReuseAddr();
		RecvSocket->SetNonBlocking();
		RecvSocket->SetRecvErr();
		// Bind to our listen port
		if (!RecvSocket->Bind(*Addr))
		{
			SocketSubsystem->DestroySocket(RecvSocket);
			RecvSocket = nullptr;
			UE_LOGF(LogAppleARKitFace, Warning, "Failed to bind to the listen port (%ls) for LiveLink face AR receiving with error (%ls)",
				*Addr->ToString(true), SocketSubsystem->GetSocketError());
		}
	}
	return RecvSocket != nullptr;
}

void FAppleARKitLiveLinkRemoteListener::Tick(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_FaceAR_ReceiveLiveLink);

	uint32 BytesPending = 0;
	while (RecvSocket->HasPendingData(BytesPending))
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get();
		TSharedRef<FInternetAddr> Sender = SocketSubsystem->CreateInternetAddr();

		int32 BytesRead = 0;
		if (RecvSocket->RecvFrom(RecvBuffer.GetData(), RecvBuffer.Num(), BytesRead, *Sender) &&
			// Make sure the packet is a complete one and ignore if it is not
			BytesRead > MIN_BLEND_SHAPE_PACKET_SIZE)
		{
			INC_DWORD_STAT(STAT_FaceAR_LiveLink_PacketsRecv);
			INC_DWORD_STAT(STAT_FaceAR_Total_LiveLink_PacketsRecv);
			INC_DWORD_STAT_BY(STAT_FaceAR_LiveLink_BytesRecv, BytesRead);
			INC_DWORD_STAT_BY(STAT_FaceAR_Total_LiveLink_BytesRecv, BytesRead);

			uint8 PacketVer = 0;
			FName SubjectName;
			uint8 BlendShapeCount = (uint8)EARFaceBlendShape::MAX;
			FName DeviceId;
			FQualifiedFrameTime FrameTime;

			FNboSerializeFromBuffer FromBuffer(RecvBuffer.GetData(), BytesRead);

			FromBuffer >> PacketVer;
			if (FromBuffer.HasOverflow() || PacketVer != BLEND_SHAPE_PACKET_VER)
			{
				UE_LOGF(LogAppleARKitFace, Verbose, "Packet overflow reading the packet version for the face AR packet");
				return;
			}
			FromBuffer >> DeviceId;
			FromBuffer >> SubjectName;
			FromBuffer >> FrameTime;
			FromBuffer >> BlendShapeCount;
			if (FromBuffer.HasOverflow() || BlendShapeCount != (uint8)EARFaceBlendShape::MAX)
			{
				UE_LOGF(LogAppleARKitFace, Verbose, "Packet overflow reading the face AR packet's non-array fields");
				return;
			}

			// Loop through and parse each float for each enum
			for (uint8 BlendShapeIndex = 0; BlendShapeIndex < BlendShapeCount && !FromBuffer.HasOverflow(); BlendShapeIndex++)
			{
				float Value = 0.f;
				FromBuffer >> Value;
				BlendShapes.Add((EARFaceBlendShape)BlendShapeIndex, Value);
			}
			// All of the data was valid, so publish it
			if (!FromBuffer.HasOverflow())
			{
				if (TSharedPtr<ILiveLinkSourceARKit> ArkitSource = Source.Pin())
				{
					if (ArkitSource->IsSourceStillValid())
					{
						ArkitSource->PublishBlendShapes(SubjectName, FrameTime, BlendShapes, DeviceId);
					}
				}
			}
			else
			{
				UE_LOGF(LogAppleARKitFace, Verbose, "Packet overflow reading the face AR packet's array of blend shapes");
			}
		}
	}
}

FAppleARKitLiveLinkFileWriter::FAppleARKitLiveLinkFileWriter(const TCHAR* InFileExtension)
	: FileExtension(InFileExtension)
{
	UAppleARKitSettings::CreateFaceTrackingLogDir();
}

void FAppleARKitLiveLinkFileWriter::Shutdown()
{
	// Save on close if desired
	if (!GetMutableDefault<UAppleARKitSettings>()->ShouldFaceTrackingLogPerFrame())
	{
		SaveFileData();
	}
}

void FAppleARKitLiveLinkFileWriter::SetTimecodeProvider(UTimecodeProvider* InTimecodeProvider)
{
	TimecodeProvider = InTimecodeProvider;
}

void FAppleARKitLiveLinkFileWriter::SaveFileData()
{
	FString SaveData;

	SaveData = BuildSaveData();
	// Write the data to the user directory
	FFileHelper::SaveStringToFile(SaveData, *GenerateFilePath(), FFileHelper::EEncodingOptions::ForceAnsi);

	FrameHistory.Empty();
}

FString FAppleARKitLiveLinkFileWriter::GenerateFilePath()
{
	FTimecode Timecode = TimecodeProvider->GetTimecode();
	FDateTime DateTime = FDateTime::UtcNow();
	const FString UserDir = FPlatformProcess::UserDir();
	const FString DeviceNameString = DeviceName.ToString();
	const FString& FaceDir = GetMutableDefault<UAppleARKitSettings>()->GetFaceTrackingLogDir();
	const TCHAR* SubDir = FaceDir.Len() > 0 ? *FaceDir : TEXT("FaceTracking");
	return FString::Printf(TEXT("%s%s/%s_%d-%d-%d-%d-%d-%d-%d%s"), *UserDir, SubDir, *DeviceNameString,
		DateTime.GetYear(), DateTime.GetMonth(), DateTime.GetDay(), Timecode.Hours, Timecode.Minutes, Timecode.Seconds, Timecode.Frames,
		*FileExtension);
}

void FAppleARKitLiveLinkFileWriter::PublishBlendShapes(FName SubjectName, const FQualifiedFrameTime& FrameTime, const FARBlendShapeMap& FaceBlendShapes, FName DeviceId)
{
	FScopeLock ScopeLock(&CriticalSection);

	if (!GetMutableDefault<UAppleARKitSettings>()->IsFaceTrackingLoggingEnabled())
	{
		return;
	}

	DeviceName = DeviceId;
	// Add to the array for long running save
	new(FrameHistory) FFaceTrackingFrame(FrameTime, FaceBlendShapes);

	if (GetMutableDefault<UAppleARKitSettings>()->ShouldFaceTrackingLogPerFrame())
	{
		SaveFileData();
	}
}

bool FAppleARKitLiveLinkFileWriter::Exec_Runtime(UWorld*, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("FaceAR")))
	{
		if (FParse::Command(&Cmd, TEXT("WriteCurveFile")))
		{
			FScopeLock ScopeLock(&CriticalSection);
			SaveFileData();
			return true;
		}
	}
	return false;
}

FAppleARKitLiveLinkFileWriterCsv::FAppleARKitLiveLinkFileWriterCsv()
	: FAppleARKitLiveLinkFileWriter(TEXT(".csv"))
{
	// Touching UObjects, so needs to be game thread
	check(IsInGameThread());

	CsvFrameHeader = TEXT("Timecode, FrameRate");
	const UEnum *EnumPtr = StaticEnum<EARFaceBlendShape>();
	if (EnumPtr != nullptr)
	{
		// Iterate through all of the enum values generating strings for them for CSV/JSON generation
		for (int32 Shape = 0; Shape < (int32) EARFaceBlendShape::MAX; Shape++)
		{
			FName EnumName = ParseEnumName(EnumPtr->GetNameByValue(Shape));
			FString EnumString(EnumName.ToString());
			CsvFrameHeader += TEXT(", ") + EnumString;
		}
	}
	CsvFrameHeader += TEXT("\r\n");
}


FString FAppleARKitLiveLinkFileWriterCsv::BuildCsvRow(const FFaceTrackingFrame& Frame)
{
	FTimecode Timecode;
	double Rate = 1.0;
	double FramesAsDecimal = 0.0;
	
	if (Frame.FrameTime.Rate.IsValid())
	{
		Timecode = FTimecode::FromFrameNumber(Frame.FrameTime.Time.GetFrame(), Frame.FrameTime.Rate);
		Rate = Frame.FrameTime.Rate.AsDecimal();
		FramesAsDecimal = double(Timecode.Frames) + Frame.FrameTime.Time.GetSubFrame();
	}

	FString SaveData = FString::Printf(TEXT("%02d:%02d:%02d:%02.3f, %.3f"), 
		Timecode.Hours, Timecode.Minutes, Timecode.Seconds, FramesAsDecimal, Rate);

	// Add all of the blend shapes on
	for (int32 Shape = 0; Shape < (int32)EARFaceBlendShape::MAX; Shape++)
	{
		float Value = Frame.BlendShapes.FindChecked((EARFaceBlendShape)Shape);
		SaveData += FString::Printf(TEXT(", %.3f"), Value);
	}
	SaveData += TEXT("\r\n");
	return SaveData;
}

FString FAppleARKitLiveLinkFileWriterCsv::BuildSaveData()
{
	FString SaveData(CsvFrameHeader);

	// Iterate through the array building our string up
	for (const FFaceTrackingFrame& Frame : FrameHistory)
	{
		SaveData += BuildCsvRow(Frame);
	}

	return SaveData;
}

FAppleARKitLiveLinkFileWriterJson::FAppleARKitLiveLinkFileWriterJson()
	: FAppleARKitLiveLinkFileWriter(TEXT(".json"))
{
	// Touching UObjects, so needs to be game thread
	check(IsInGameThread());

	const UEnum *EnumPtr = StaticEnum<EARFaceBlendShape>();
	if (EnumPtr != nullptr)
	{
		// Iterate through all of the enum values generating strings for them for CSV/JSON generation
		for (int32 Shape = 0; Shape < (int32) EARFaceBlendShape::MAX; Shape++)
		{
			FName EnumName = ParseEnumName(EnumPtr->GetNameByValue(Shape));
			FString EnumString(EnumName.ToString());
			BlendShapeJsonKeyNames.Add(EnumString);
		}
	}
}

FString FAppleARKitLiveLinkFileWriterJson::BuildJsonRow(const FFaceTrackingFrame& Frame)
{
	FTimecode Timecode;
	double Rate = 1.0;
	double FramesAsDecimal = 0.0;

	if (Frame.FrameTime.Rate.IsValid())
	{
		Timecode = FTimecode::FromFrameNumber(Frame.FrameTime.Time.GetFrame(), Frame.FrameTime.Rate);
		Rate = Frame.FrameTime.Rate.AsDecimal();
		FramesAsDecimal = double(Timecode.Frames) + Frame.FrameTime.Time.GetSubFrame();
	}

	FString SaveData = FString::Printf(TEXT("\t{\r\n\t\t\"TimeCode\" :\r\n\t\t{\r\n\t\t\t\"Hours\" : %d,\r\n\t\t\t\"Minutes\" : %d,\r\n\t\t\t\"Seconds\" : %d,\r\n\t\t\t\"Frames\" : %.3f\r\n\t\t},\r\n"),
			Timecode.Hours, Timecode.Minutes, Timecode.Seconds, FramesAsDecimal);
	SaveData += FString::Printf(TEXT("\t\t\"FrameRate\" : %.3f,\r\n"), Rate);
	bool bNeedsComma = false;
	// Add all of the blend shapes on
	for (int32 Shape = 0; Shape < (int32)EARFaceBlendShape::MAX; Shape++)
	{
		if (bNeedsComma)
		{
			SaveData += TEXT(",\r\n");
		}
		float Value = Frame.BlendShapes.FindChecked((EARFaceBlendShape)Shape);
		SaveData += FString::Printf(TEXT("\t\t\"%s\" : %.3f"), *BlendShapeJsonKeyNames[Shape], Value);
		bNeedsComma = true;
	}
	SaveData += TEXT("\r\n\t}");
	return SaveData;
}

FString FAppleARKitLiveLinkFileWriterJson::BuildSaveData()
{
	FString SaveData;
	SaveData += TEXT("{\r\n");
	SaveData += TEXT("\t\"Frames\" : [\r\n");
	bool bNeedsComma = false;
	// Add each frame
	for (const FFaceTrackingFrame& Frame : FrameHistory)
	{
		if (bNeedsComma)
		{
			SaveData += TEXT(",\r\n");
		}
		SaveData += BuildJsonRow(Frame);
		bNeedsComma = true;
	}
	SaveData += TEXT("\t]\r\n");
	SaveData += TEXT("}\r\n");
	return SaveData;
}
