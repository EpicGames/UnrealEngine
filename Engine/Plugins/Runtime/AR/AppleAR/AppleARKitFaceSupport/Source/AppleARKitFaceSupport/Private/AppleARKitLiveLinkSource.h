// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AppleARKitLiveLinkConnectionSettings.h"
#include "AppleARKitLiveLinkSourceFactory.h"
#include "ILiveLinkClient.h"
#include "LiveLinkTypes.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "Tickable.h"
#include "ARTrackable.h"

#include "IPAddress.h"
#include "Sockets.h"
//#include "NboSerializer.h"

/**
 * Hack because our build process doesn't handle plugin dependencies correctly. This will be removed
 */

/**
 * Serializes data in network byte order form into a buffer
 */
class FNboSerializeToBuffer
{
	/** Hidden on purpose */
	FNboSerializeToBuffer(void);

protected:
	/**
	 * Holds the data as it is serialized
	 */
	TArray<uint8> Data;
	/**
	 * Tracks how many bytes have been written in the packet
	 */
	uint32 NumBytes;

	/** Indicates whether writing to the buffer caused an overflow or not */
	bool bHasOverflowed;

public:
	/**
	 * Inits the write tracking
	 */
	FNboSerializeToBuffer(uint32 Size) :
			NumBytes(0),
			bHasOverflowed(false)
	{
		Data.Empty(Size);
		Data.AddZeroed(Size);
	}

	/**
	 * Cast operator to get at the formatted buffer data
	 */
	inline operator uint8*(void) const
	{
		return (uint8*)Data.GetData();
	}

	/**
	 * Cast operator to get at the formatted buffer data
	 */
	inline const TArray<uint8>& GetBuffer(void) const
	{
		return Data;
	}

	/**
	 * Returns the size of the buffer we've built up
	 */
	inline uint32 GetByteCount(void) const
	{
		return NumBytes;
	}

	/**
	 * Returns the number of bytes preallocated in the array
	 */
	inline uint32 GetBufferSize(void) const
	{
		return (uint32)Data.Num();
	}

	/**
	 * Trim any extra space in the buffer that is not used
	 */
	inline void TrimBuffer()
	{
		if (GetBufferSize() > GetByteCount())
		{
			Data.RemoveAt(GetByteCount(), GetBufferSize()-GetByteCount());
		}
	}

	/**
	 * Act as if the buffer is now empty. Useful for buffers that are reused
	 */
	inline void Reset()
	{
		NumBytes = 0;
		bHasOverflowed = false;
	}

	/**
	 * Adds a char to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const char Ch)
	{
		if (!Ar.HasOverflow() && Ar.NumBytes + 1 <= Ar.GetBufferSize())
		{
			Ar.Data[Ar.NumBytes++] = Ch;
		}
		else
		{
			Ar.bHasOverflowed = true;
		}

		return Ar;
	}

	/**
	 * Adds a byte to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const uint8& B)
	{
		if (!Ar.HasOverflow() && Ar.NumBytes + 1 <= Ar.GetBufferSize())
		{
			Ar.Data[Ar.NumBytes++] = B;
		}
		else
		{
			Ar.bHasOverflowed = true;
		}

		return Ar;
	}

	/**
	 * Adds a byte to the buffer
	 */
	template<class TEnum>
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const TEnumAsByte<TEnum>& B)
	{
		return Ar << *(uint8*)&B;
	}

	/**
	 * Adds an int32 to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const int32& I)
	{
		return Ar << *(uint32*)&I;
	}

	/**
	 * Adds a uint32 to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const uint32& D)
	{
		if (!Ar.HasOverflow() && Ar.NumBytes + 4 <= Ar.GetBufferSize())
		{
			Ar.Data[Ar.NumBytes + 0] = (D >> 24) & 0xFF;
			Ar.Data[Ar.NumBytes + 1] = (D >> 16) & 0xFF;
			Ar.Data[Ar.NumBytes + 2] = (D >> 8) & 0xFF;
			Ar.Data[Ar.NumBytes + 3] = D & 0xFF;
			Ar.NumBytes += 4;
		}
		else
		{
			Ar.bHasOverflowed = true;
		}

		return Ar;
	}

	/**
	 * Adds a uint64 to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const uint64& Q)
	{
		if (!Ar.HasOverflow() && Ar.NumBytes + 8 <= Ar.GetBufferSize())
		{
			Ar.Data[Ar.NumBytes + 0] = (Q >> 56) & 0xFF;
			Ar.Data[Ar.NumBytes + 1] = (Q >> 48) & 0xFF;
			Ar.Data[Ar.NumBytes + 2] = (Q >> 40) & 0xFF;
			Ar.Data[Ar.NumBytes + 3] = (Q >> 32) & 0xFF;
			Ar.Data[Ar.NumBytes + 4] = (Q >> 24) & 0xFF;
			Ar.Data[Ar.NumBytes + 5] = (Q >> 16) & 0xFF;
			Ar.Data[Ar.NumBytes + 6] = (Q >> 8) & 0xFF;
			Ar.Data[Ar.NumBytes + 7] = Q & 0xFF;
			Ar.NumBytes += 8;
		}
		else
		{
			Ar.bHasOverflowed = true;
		}

		return Ar;
	}

	/**
	 * Adds a float to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const float& F)
	{
		uint32 Value = *(uint32*)&F;
		return Ar << Value;
	}

	/**
	 * Adds a double to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const double& Dbl)
	{
		uint64 Value = *(uint64*)&Dbl;
		return Ar << Value;
	}

	/**
	 * Adds a FString to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const FString& String)
	{
		// We send strings length prefixed
		FTCHARToUTF8 Converted(*String);
		int32 Len = Converted.Length();

		Ar << Len;

		if (!Ar.HasOverflow() && Ar.NumBytes + Len <= Ar.GetBufferSize())
		{
			// Handle empty strings
			if (Len > 0)
			{
				ANSICHAR* Ptr = (ANSICHAR*)Converted.Get();

				// memcpy it into the buffer
				FMemory::Memcpy(&Ar.Data[Ar.NumBytes], Ptr,Len);
				Ar.NumBytes += Len;
			}
		}
		else
		{
			Ar.bHasOverflowed = true;
		}

		return Ar;
	}

	/**
	 * Adds a string to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const TCHAR* String)
	{
		// We send strings length prefixed (conversion handles null pointers)
		FTCHARToUTF8 Converted(String);
		int32 Len = Converted.Length();

		Ar << Len;

		if (!Ar.HasOverflow() && Ar.NumBytes + Len <= Ar.GetBufferSize())
		{
			// Handle empty/null strings
			if (Len > 0)
			{
				ANSICHAR* Ptr = (ANSICHAR*)Converted.Get();

				// memcpy it into the buffer
				FMemory::Memcpy(&Ar.Data[Ar.NumBytes], Ptr, Len);
				Ar.NumBytes += Len;
			}
		}
		else
		{
			Ar.bHasOverflowed = true;
		}

		return Ar;
	}

	/**
	 * Adds a substring to the buffer
	 */
	inline FNboSerializeToBuffer& AddString(const ANSICHAR* String,const int32 Length)
	{
		// We send strings length prefixed
		int32 Len = Length;
		(*this) << Len;

		if (!HasOverflow() && NumBytes + Len <= GetBufferSize())
		{
			// Don't process if null
			if (String)
			{
				// memcpy it into the buffer
				FMemory::Memcpy(&Data[NumBytes],String,Len);
				NumBytes += Len;
			}
		}
		else
		{
			bHasOverflowed = true;
		}

		return *this;
	}

	/**
	 * Adds an FName to the buffer as a string
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const FName& Name)
	{
		const FString NameString = Name.ToString();
		Ar << NameString;
		return Ar;
	}


	/**
	 * Adds an ip address to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const FInternetAddr& Addr)
	{
		uint32 OutIp;
		Addr.GetIp(OutIp);
		Ar << OutIp;
		int32 OutPort;
		Addr.GetPort(OutPort);
		Ar << OutPort;
		return Ar;
	}

	/**
	 * Adds a FGuid to the buffer
	 */
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar,const FGuid& Guid)
	{
		Ar << Guid.A;
		Ar << Guid.B;
		Ar << Guid.C;
		Ar << Guid.D;
		return Ar;
	}

	/**
	* Adds a FQualifiedFrameTime to the buffer
	*/
	friend inline FNboSerializeToBuffer& operator<<(FNboSerializeToBuffer& Ar, const FQualifiedFrameTime& FrameTime)
	{
		Ar << FrameTime.Time.FrameNumber.Value;
		Ar << FrameTime.Time.GetSubFrame();
		Ar << FrameTime.Rate.Numerator;
		Ar << FrameTime.Rate.Denominator;
		return Ar;
	}

	/**
	 * Writes a blob of data to the buffer
	 *
	 * @param Buffer the source data to append
	 * @param NumToWrite the size of the blob to write
	 */
	inline void WriteBinary(const uint8* Buffer,uint32 NumToWrite)
	{
		if (!HasOverflow() && NumBytes + NumToWrite <= GetBufferSize())
		{
			FMemory::Memcpy(&Data[NumBytes],Buffer,NumToWrite);
			NumBytes += NumToWrite;
		}
		else
		{
			bHasOverflowed = true;
		}
	}

	/**
	 * Gets the buffer at a specific point
	 */
	inline uint8* GetRawBuffer(uint32 Offset)
	{
		return &Data[Offset];
	}

	/**
	 * Skips forward in the buffer by the specified amount
	 *
	 * @param Amount the number of bytes to skip ahead
	 */
	inline void SkipAheadBy(uint32 Amount)
	{
		if (!HasOverflow() && NumBytes + Amount <= GetBufferSize())
		{
			NumBytes += Amount;
		}
		else
		{
			bHasOverflowed = true;
		}
	}

	/** Returns whether the buffer had an overflow when writing to it */
	inline bool HasOverflow(void) const
	{
		return bHasOverflowed;
	}
};

MSVC_PRAGMA(warning(push))
// Disable used without initialization warning because the reads are initializing
MSVC_PRAGMA(warning(disable : 4700))

/**
 * Class used to read data from a NBO data buffer
 */
class FNboSerializeFromBuffer
{
protected:
	/** Pointer to the data this reader is attached to */
	const uint8* Data;
	/** The size of the data in bytes */
	const int32 NumBytes;
	/** The current location in the byte stream for reading */
	int32 CurrentOffset;
	/** Indicates whether reading from the buffer caused an overflow or not */
	bool bHasOverflowed;

	/** Hidden on purpose */
	FNboSerializeFromBuffer(void);

public:
	/**
	 * Initializes the buffer, size, and zeros the read offset
	 *
	 * @param InData the buffer to attach to
	 * @param Length the size of the buffer we are attaching to
	 */
	FNboSerializeFromBuffer(const uint8* InData,int32 Length) :
			Data(InData),
			NumBytes(Length),
			CurrentOffset(0),
			bHasOverflowed(false)
	{
	}

	/**
	 * Reads a char from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,char& Ch)
	{
		if (!Ar.HasOverflow() && Ar.CurrentOffset + 1 <= Ar.NumBytes)
		{
			Ch = Ar.Data[Ar.CurrentOffset++];
		}
		else
		{
			Ar.bHasOverflowed = true;
		}
		return Ar;
	}

	/**
	 * Reads a byte from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,uint8& B)
	{
		if (!Ar.HasOverflow() && Ar.CurrentOffset + 1 <= Ar.NumBytes)
		{
			B = Ar.Data[Ar.CurrentOffset++];
		}
		else
		{
			Ar.bHasOverflowed = true;
		}
		return Ar;
	}
	/**
	 * Reads a byte from the buffer
	 */
	template<class TEnum>
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,TEnumAsByte<TEnum>& B)
	{
		return Ar >> *(uint8*)&B;
	}

	/**
	 * Reads an int32 from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,int32& I)
	{
		return Ar >> *(uint32*)&I;
	}

	/**
	 * Reads a uint32 from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,uint32& D)
	{
		if (!Ar.HasOverflow() && Ar.CurrentOffset + 4 <= Ar.NumBytes)
		{
			uint32 D1 = Ar.Data[Ar.CurrentOffset + 0];
			uint32 D2 = Ar.Data[Ar.CurrentOffset + 1];
			uint32 D3 = Ar.Data[Ar.CurrentOffset + 2];
			uint32 D4 = Ar.Data[Ar.CurrentOffset + 3];
			D = D1 << 24 | D2 << 16 | D3 << 8 | D4;
			Ar.CurrentOffset += 4;
		}
		else
		{
			Ar.bHasOverflowed = true;
		}
		return Ar;
	}

	/**
	 * Adds a uint64 to the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,uint64& Q)
	{
		if (!Ar.HasOverflow() && Ar.CurrentOffset + 8 <= Ar.NumBytes)
		{
			Q = ((uint64)Ar.Data[Ar.CurrentOffset + 0] << 56) |
				((uint64)Ar.Data[Ar.CurrentOffset + 1] << 48) |
				((uint64)Ar.Data[Ar.CurrentOffset + 2] << 40) |
				((uint64)Ar.Data[Ar.CurrentOffset + 3] << 32) |
				((uint64)Ar.Data[Ar.CurrentOffset + 4] << 24) |
				((uint64)Ar.Data[Ar.CurrentOffset + 5] << 16) |
				((uint64)Ar.Data[Ar.CurrentOffset + 6] << 8) |
				(uint64)Ar.Data[Ar.CurrentOffset + 7];
			Ar.CurrentOffset += 8;
		}
		else
		{
			Ar.bHasOverflowed = true;
		}
		return Ar;
	}

	/**
	 * Reads a float from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,float& F)
	{
		return Ar >> *(uint32*)&F;
	}

	/**
	 * Reads a double from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,double& Dbl)
	{
		return Ar >> *(uint64*)&Dbl;
	}

	/**
	 * Reads a FString from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,FString& String)
	{
		// We send strings length prefixed
		int32 Len = 0;
		Ar >> Len;

		// Check this way to trust NumBytes and CurrentOffset to be more accurate than the packet Len value
		const bool bSizeOk = (Len >= 0) && (Len <= (Ar.NumBytes - Ar.CurrentOffset));
		if (!Ar.HasOverflow() && bSizeOk)
		{
			// Handle strings of zero length
			if (Len > 0)
			{
				char* Temp = (char*)FMemory_Alloca(Len + 1);
				// memcpy it in from the buffer
				FMemory::Memcpy(Temp, &Ar.Data[Ar.CurrentOffset], Len);
				Temp[Len] = '\0';

				FUTF8ToTCHAR Converted(Temp);
				TCHAR* Ptr = (TCHAR*)Converted.Get();
				String = Ptr;
				Ar.CurrentOffset += Len;
			}
			else
			{
				String.Empty();
			}
		}
		else
		{
			Ar.bHasOverflowed = true;
		}

		return Ar;
	}

	/**
	 * Reads an FName from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,FName& Name)
	{
		FString NameString;
		Ar >> NameString;
		Name = FName(*NameString);
		return Ar;
	}

	/**
	 * Reads an ip address from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,FInternetAddr& Addr)
	{
		uint32 InIp = 0;
		Ar >> InIp;
		Addr.SetIp(InIp);
		int32 InPort = 0;
		Ar >> InPort;
		Addr.SetPort(InPort);
		return Ar;
	}

	/**
	 * Reads a FGuid from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar,FGuid& Guid)
	{
		Ar >> Guid.A;
		Ar >> Guid.B;
		Ar >> Guid.C;
		Ar >> Guid.D;
		return Ar;
	}

	/**
	 * Reads a FQualifiedFrameTime from the buffer
	 */
	friend inline FNboSerializeFromBuffer& operator>>(FNboSerializeFromBuffer& Ar, FQualifiedFrameTime& FrameTime)
	{
		int32 FrameNumber;
		float SubFrame;

		Ar >> FrameNumber;
		Ar >> SubFrame;
		Ar >> FrameTime.Rate.Numerator;
		Ar >> FrameTime.Rate.Denominator;

		FrameTime.Time = FFrameTime(FrameNumber, SubFrame);

		return Ar;
	}

	/**
	 * Reads a blob of data from the buffer
	 *
	 * @param OutBuffer the destination buffer
	 * @param NumToRead the size of the blob to read
	 */
	void ReadBinary(uint8* OutBuffer,uint32 NumToRead)
	{
		if (!HasOverflow() && CurrentOffset + (int32)NumToRead <= NumBytes)
		{
			FMemory::Memcpy(OutBuffer,&Data[CurrentOffset],NumToRead);
			CurrentOffset += NumToRead;
		}
		else
		{
			bHasOverflowed = true;
		}
	}

	/**
	 * Seek to the desired position in the buffer
	 *
	 * @param Pos the offset from the start of the buffer
	 */
	void Seek(int32 Pos)
	{
		checkSlow(Pos >= 0);

		if (!HasOverflow() && Pos < NumBytes)
		{
			CurrentOffset = Pos;
		}
		else
		{
			bHasOverflowed = true;
		}
	}

	/** @return Current position of the buffer being to be read */
	inline int32 Tell(void) const
	{
		return CurrentOffset;
	}

	/** Returns whether the buffer had an overflow when reading from it */
	inline bool HasOverflow(void) const
	{
		return bHasOverflowed;
	}

	/** @return Number of bytes remaining to read from the current offset to the end of the buffer */
	inline int32 AvailableToRead(void) const
	{
		return FMath::Max<int32>(0,NumBytes - CurrentOffset);
	}

	/**
	 * Returns the number of total bytes in the buffer
	 */
	inline int32 GetBufferSize(void) const
	{
		return NumBytes;
	}
};

MSVC_PRAGMA(warning(pop))

/**
 * End hack
 */

class UARFaceGeometry;

/** Listens for LiveLink publication events and publishes them to the local LiveLinkClient */
class FAppleARKitLiveLinkRemoteListener :
	public FTickableGameObject
{
public:
	FAppleARKitLiveLinkRemoteListener(const TSharedPtr<ILiveLinkSourceARKit>& ArkitSource);
	virtual ~FAppleARKitLiveLinkRemoteListener();

	// FTickableGameObject interface
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override { return RecvSocket != nullptr; }
	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAppleARKitLiveLinkRemoteListener, STATGROUP_Tickables);
	}
	virtual bool IsTickableInEditor() const override { return true; }
	virtual bool IsTickableWhenPaused() const override { return true; }
	// End FTickableGameObject

	bool InitReceiveSocket(int32 Port);

private:

	/** Socket that is read from to publish remote data to this local LiveLinkClient */
	FSocket* RecvSocket;
	/** Buffer to reuse for receiving data */
	TArray<uint8> RecvBuffer;
	/** The reused blend shape map to avoid allocs/frees */
	FARBlendShapeMap BlendShapes;

	/** The source used to publish events from the remote socket */
	TWeakPtr<ILiveLinkSourceARKit> Source;
};

/** Forwards face blend shapes to another machine for LiveLink publication */
class FAppleARKitLiveLinkRemotePublisher :
	public IARKitBlendShapePublisher
{
public:
	FAppleARKitLiveLinkRemotePublisher();
	FAppleARKitLiveLinkRemotePublisher(const FString& InRemoteIp);
	virtual ~FAppleARKitLiveLinkRemotePublisher();

	// IARKitBlendShapePublisher interface
	virtual void PublishBlendShapes(FName SubjectName, const FQualifiedFrameTime& FrameTime, const FARBlendShapeMap& FaceBlendShapes, FName DeviceId = NAME_None) override;
	virtual void Shutdown() override {}
	// End IARKitBlendShapePublisher

	bool InitSendSocket();

private:
	TSharedRef<FInternetAddr> GetSendAddress();

	/** The IP address to send the blend shapes to */
	FString RemoteIp;
	/** Socket to send the data via to the remote listener */
	FSocket* SendSocket;
	/** The ram buffer to serialize the packet to */
	FNboSerializeToBuffer SendBuffer;
};

/** Writes files to disk with the face tracking information */
class FAppleARKitLiveLinkFileWriter :
	public FSelfRegisteringExec,
	public IARKitBlendShapePublisher
{
public:
	virtual ~FAppleARKitLiveLinkFileWriter() {}
	virtual void SetTimecodeProvider(UTimecodeProvider* InTimecodeProvider = nullptr) override;
	virtual void Shutdown() override;

protected:
	FAppleARKitLiveLinkFileWriter(const TCHAR* InFileExtension);

	struct FFaceTrackingFrame
	{
		FFaceTrackingFrame(const FQualifiedFrameTime& InFrameTime, const FARBlendShapeMap& InBlendShapes)
			: FrameTime(InFrameTime)
			, BlendShapes(InBlendShapes)
		{
		}

		FQualifiedFrameTime FrameTime;
		FARBlendShapeMap BlendShapes;
	};

	// IARKitBlendShapePublisher interface
	virtual void PublishBlendShapes(FName SubjectName, const FQualifiedFrameTime& FrameTime, const FARBlendShapeMap& FaceBlendShapes, FName DeviceId = NAME_None) override;
	// End IARKitBlendShapePublisher

	//~ FSelfRegisteringExec
	virtual bool Exec_Runtime(UWorld*, const TCHAR* Cmd, FOutputDevice& Ar) override;
	//~ FSelfRegisteringExec

	virtual FString BuildSaveData() = 0;

	void SaveFileData();
	FString GenerateFilePath();

	/** Needed because Exec commands are on the game thread and updates are on an arbitrary thread */
	FCriticalSection CriticalSection;
	/** The file extension to write to */
	FString FileExtension;
	/** The list of the last frames seen. Used when rolling up all data in one file */
	TArray<FFaceTrackingFrame> FrameHistory;
	/** The name to use when generating the file to save to */
	FName DeviceName;
	/**
	 * The time code provider to use when tagging file time stamps
	 * Note: this requires the FAppleARKitSystem object to mark it in use so GC doesn't destroy it. Normally it would
	 * implement the FGCObject interface but this gets created before UObjects are init-ed so not possible
	 * */
	UTimecodeProvider* TimecodeProvider;
};

class FAppleARKitLiveLinkFileWriterJson :
	public FAppleARKitLiveLinkFileWriter
{
public:
	FAppleARKitLiveLinkFileWriterJson();
	virtual ~FAppleARKitLiveLinkFileWriterJson() {}

private:
	//~ FAppleARKitLiveLinkFileWriter
	virtual FString BuildSaveData() override;
	FString BuildJsonRow(const FFaceTrackingFrame& Frame);
	//~ FAppleARKitLiveLinkFileWriter

	/** A list of the keys to use when generating JSON files, cached because can't access off game thread */
	TArray<FString> BlendShapeJsonKeyNames;
};

class FAppleARKitLiveLinkFileWriterCsv :
	public FAppleARKitLiveLinkFileWriter
{
public:
	FAppleARKitLiveLinkFileWriterCsv();
	virtual ~FAppleARKitLiveLinkFileWriterCsv() {}

private:
	//~ FAppleARKitLiveLinkFileWriter
	virtual FString BuildSaveData() override;
	FString BuildCsvRow(const FFaceTrackingFrame& Frame);
	//~ FAppleARKitLiveLinkFileWriter

	/** A pre-made header to use when writing CSV files, cached because can't access off game thread  */
	FString CsvFrameHeader;
};

/** Publishes face blend shapes to LiveLink for use locally */
class FAppleARKitLiveLinkSource :
	public ILiveLinkSourceARKit, public TSharedFromThis<FAppleARKitLiveLinkSource>
{
public:
	FAppleARKitLiveLinkSource();
	// Constructor invoked from the AppleARKitLiveLinkSourceFactory wit the settings specified by the user.
	FAppleARKitLiveLinkSource(FAppleARKitLiveLinkConnectionSettings InConnectionSettings);
	virtual ~FAppleARKitLiveLinkSource() {}

	// Bind the remote listener to the port provided by this source's connection settings.
	void InitializeRemoteListener();

	virtual void Shutdown() override {}
private:
	// ILiveLinkSource interface
	virtual void ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) override;
	virtual void InitializeSettings(ULiveLinkSourceSettings* Settings) override;
	virtual bool IsSourceStillValid() const override;
	virtual bool RequestSourceShutdown() override;
	virtual FText GetSourceMachineName() const override;
	virtual FText GetSourceStatus() const override;
	virtual FText GetSourceType() const override;
	// End ILiveLinkSource

	// IARKitBlendShapePublisher interface
	virtual void PublishBlendShapes(FName SubjectName, const FQualifiedFrameTime& FrameTime, const FARBlendShapeMap& FaceBlendShapes, FName DeviceId = NAME_None) override;
	// End IARKitBlendShapePublisher

	// Update static data property names from blend shape map. Should only happen once per subject name. Pushes static data to livelink to register subject
	void UpdateStaticData(FName SubjectName, const FARBlendShapeMap& FaceBlendShapes, FName DeviceId);

private:

	/** The local client to push data updates to */
	ILiveLinkClient* Client;

	/** Our identifier in LiveLink */
	FGuid SourceGuid;

	/** The last time we sent the data. Used to not send redundant data */
	uint32 LastFramePublished;

	/** Helper struct to contain BlendShape data per deviceId */
	struct FBlendShapeStaticData
	{
		FLiveLinkSubjectKey SubjectKey;
		FLiveLinkSkeletonStaticData StaticData;
	};

	/** Used to track names changes for a given device and keep track of property names contained in subject */
	TMap<FName, FBlendShapeStaticData> BlendShapePerDeviceMap;

	/** Used to listen on a socket for incoming face data and publishing this data to livelink. */
	TUniquePtr<FAppleARKitLiveLinkRemoteListener> RemoteListener;

	/** Connection settings for this source. */
	FAppleARKitLiveLinkConnectionSettings ConnectionSettings;
};
