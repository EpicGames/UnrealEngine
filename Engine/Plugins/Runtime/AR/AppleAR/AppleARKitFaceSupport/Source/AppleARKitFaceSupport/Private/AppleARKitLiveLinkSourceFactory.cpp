// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleARKitLiveLinkSourceFactory.h"

#include "AppleARKitLiveLinkConnectionSettings.h"
#include "AppleARKitLiveLinkSource.h"
#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "SAppleARKitLiveLinkSourceFactory.h"

#define LOCTEXT_NAMESPACE "AppleARKitLiveLinkSourceFactory"

FText UAppleARKitLiveLinkSourceFactory::GetSourceDisplayName() const
{
	return LOCTEXT("SourceDisplayName", "Apple ARKit Source");
}

FText UAppleARKitLiveLinkSourceFactory::GetSourceTooltip() const
{
	return LOCTEXT("SourceTooltip", "Allows creation of an AppleARKit LiveLink source. ");
}

TSharedPtr<SWidget> UAppleARKitLiveLinkSourceFactory::BuildCreationPanel(FOnLiveLinkSourceCreated InOnLiveLinkSourceCreated) const
{
	return SNew(SAppleARKitLiveLinkSourceFactory)
		.OnConnectionSettingsAccepted(FOnConnectionSettingsAccepted::CreateUObject(this, &UAppleARKitLiveLinkSourceFactory::CreateSourceFromSettings, InOnLiveLinkSourceCreated));
}

TSharedPtr<ILiveLinkSource> UAppleARKitLiveLinkSourceFactory::CreateSource(const FString& ConnectionString) const
{
	FAppleARKitLiveLinkConnectionSettings ConnectionSettings;
	if (!ConnectionString.IsEmpty())
	{
		FAppleARKitLiveLinkConnectionSettings::StaticStruct()->ImportText(*ConnectionString, &ConnectionSettings, nullptr, PPF_None, GLog, TEXT("UAppleARKitLiveLinkSourceFactory"));
	}
	
	TSharedPtr<FAppleARKitLiveLinkSource> Source = MakeShared<FAppleARKitLiveLinkSource>(ConnectionSettings);
	Source->InitializeRemoteListener();

	return Source;
}

void UAppleARKitLiveLinkSourceFactory::CreateSourceFromSettings(const FAppleARKitLiveLinkConnectionSettings& InConnectionSettings, FOnLiveLinkSourceCreated OnSourceCreated) const
{
	FString ConnectionString;
	FAppleARKitLiveLinkConnectionSettings::StaticStruct()->ExportText(ConnectionString, &InConnectionSettings, nullptr, nullptr, PPF_None, nullptr);

	TSharedPtr<FAppleARKitLiveLinkSource> SharedPtr = MakeShared<FAppleARKitLiveLinkSource>(InConnectionSettings);
	SharedPtr->InitializeRemoteListener();

	OnSourceCreated.ExecuteIfBound(SharedPtr, MoveTemp(ConnectionString));
}

#undef LOCTEXT_NAMESPACE
