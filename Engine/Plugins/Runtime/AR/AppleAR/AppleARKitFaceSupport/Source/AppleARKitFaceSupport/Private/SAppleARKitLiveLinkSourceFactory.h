// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AppleARKitLiveLinkConnectionSettings.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#if WITH_EDITOR
#include "IStructureDetailsView.h"
#endif //WITH_EDITOR

#include "Input/Reply.h"

struct FAppleARKitLiveLinkConnectionSettings;

DECLARE_DELEGATE_OneParam(FOnConnectionSettingsAccepted, const FAppleARKitLiveLinkConnectionSettings&);

class SAppleARKitLiveLinkSourceFactory : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SAppleARKitLiveLinkSourceFactory)
		{}
		SLATE_EVENT(FOnConnectionSettingsAccepted, OnConnectionSettingsAccepted)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args);


private:
	FAppleARKitLiveLinkConnectionSettings ConnectionSettings;

#if WITH_EDITOR
	TSharedPtr<FStructOnScope> StructOnScope;
	TSharedPtr<IStructureDetailsView> StructureDetailsView;
#endif //WITH_EDITOR

	FReply OnSettingsAccepted();
	FOnConnectionSettingsAccepted OnConnectionSettingsAccepted;
};