// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AppleARKitAvailability.h"
#include "ARSystem.h"
#include "ProceduralMeshComponent.h"
#include "AppleARKitLiveLinkSourceFactory.h"
#include "ARTrackable.h"
#include "AppleARKitFaceMeshComponent.generated.h"


UENUM(BlueprintType, Category="AR AugmentedReality", meta=(Experimental))
enum class EARFaceComponentTransformMixing : uint8
{
	/** Uses the component's transform exclusively. Only setting for non-tracked meshes */
	ComponentOnly,
	/** Use the component's location and apply the rotation from the tracked mesh */
	ComponentLocationTrackedRotation,
	/** Concatenate the component and the tracked face transforms */
	ComponentWithTracked,
	/** Use only the tracked face transform */
	TrackingOnly
};

/**
 * Packs the curve into 2 bytes with the amount being +/- 127
 */
USTRUCT()
struct FNetQuantizeFaceCurve
{
	GENERATED_USTRUCT_BODY()

	inline FNetQuantizeFaceCurve()
	{
	}

	inline FNetQuantizeFaceCurve(EARFaceBlendShape InBlendShape, float InAmount)
		: BlendShape((uint8)InBlendShape)
		, Amount(ConvertAmountToInt(InAmount))
	{
	}

	inline float GetAmountAsFloat() const
	{
		return ConvertAmountToFloat(Amount);
	}

	inline EARFaceBlendShape GetBlendShape() const
	{
		check(BlendShape < (uint8)EARFaceBlendShape::MAX);
		return (EARFaceBlendShape)BlendShape;
	}

	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
	{
		uint16 Value = 0;
		if (Ar.IsSaving())
		{
			Value = ((uint8)Amount | (BlendShape << 8));
			Ar << Value;
		}
		else
		{
			Ar << Value;
			Amount = (int8)(Value & 0xFF);
			BlendShape = ((Value >> 8) & 0xFF);
		}
		bOutSuccess = true;
		return true;
	}

	static bool IsDifferentEnough(float Val1, float Val2)
	{
		return ConvertAmountToInt(Val1) != ConvertAmountToInt(Val2);
	}

private:
	static constexpr float Scale = 127.f;
	static constexpr float InvScale = 1.f / Scale;
	
	static inline float ConvertAmountToFloat(int8 InAmount)
	{
		return ((float)InAmount) * InvScale;
	}

	static inline int8 ConvertAmountToInt(float InAmount)
	{
		return (int8)FMath::TruncToInt(InAmount * Scale);
	}

	uint8 BlendShape;
	int8 Amount;
};

template<>
struct TStructOpsTypeTraits< FNetQuantizeFaceCurve > : public TStructOpsTypeTraitsBase2< FNetQuantizeFaceCurve >
{
	enum
	{
		WithNetSerializer = true,
		WithNetSharedSerialization = true,
	};
};

/**
 * This component is updated by the ARSystem with face data on devices that have support for it
 */
UCLASS(hidecategories = (Object, LOD, "Components|ProceduralMesh"), meta = (BlueprintSpawnableComponent), ClassGroup = "AR")
class APPLEARKITFACESUPPORT_API UAppleARKitFaceMeshComponent :
	public UProceduralMeshComponent
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 *	Create the initial face mesh from raw mesh data
	 *
	 *	@param	Vertices			Vertex buffer of all vertex positions to use for this mesh section.
	 *	@param	Triangles			Index buffer indicating which vertices make up each triangle. Length must be a multiple of 3.
	 *	@param	UV0					Optional array of texture co-ordinates for each vertex. If supplied, must be same length as Vertices array.
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|ARFaceMesh", meta = (DisplayName = "Create Face Mesh", AutoCreateRefTerm = "UV0"))
	void CreateMesh(const TArray<FVector>& Vertices, const TArray<int32>& Triangles, const TArray<FVector2D>& UV0);

	/**
	 *	Set all of the blend shapes for this instance from a set of blend shapes
	 *
	 *	@param	BlendShapes			The set of blend shapes to conform the face mesh to
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|ARFaceMesh", meta = (DisplayName = "Create Face Mesh from Blend Shapes"))
	void SetBlendShapes(const TMap<EARFaceBlendShape, float>& InBlendShapes);

	/**
	 *	Sets the amount for a given blend shape
	 *
	 *	@param	BlendShape			The blend shape to modify
	 *	@param	Amount				The amount of the blend shape to apply (0..1)
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|ARFaceMesh", meta = (DisplayName = "Set the value of a Blend Shape"))
	void SetBlendShapeAmount(EARFaceBlendShape BlendShape, float Amount);

	/**
	 * Returns the value of the specified blend shape
	 *
	 * @param BlendShape			The blend shape to query
	 *
	 * @return the current amount the blend shape should be applied
	 */
	UFUNCTION(BlueprintPure, Category = "Components|ARFaceMesh")
	float GetFaceBlendShapeAmount(EARFaceBlendShape BlendShape) const;

	/**
	 *	Create/replace the face mesh from the current set of blend shapes if the device supports it
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|ARFaceMesh", meta = (DisplayName = "Update Face Mesh from Blend Shapes"))
	void UpdateMeshFromBlendShapes();

	/**
	 *	Updates the face mesh vertices. The topology and UVs do not change post creation so only vertices are updated
	 *
	 *	@param	Vertices			Vertex buffer of all vertex positions to use for this mesh section.
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|ARFaceMesh", meta = (DisplayName = "Update Mesh Section FColor"))
	void UpdateMesh(const TArray<FVector>& Vertices);

	/**
	 * If auto bind is true, then this component will update itself from the local face tracking data each tick. If auto bind is off, ticking is disabled
	 *
	 * @param	bAutoBind			true to enable, false to disable
	 */
	UFUNCTION(BlueprintCallable, Category = "Components|ARFaceMesh", meta = (DisplayName = "Modify auto bind to local face tracking"))
	void SetAutoBind(bool bAutoBind);

	/** Returns the frame number that was last used to update this component */
	UFUNCTION(BlueprintPure, Category = "Components|ARFaceMesh")
	int32 GetLastUpdateFrameNumber() const;

	/** Returns the frame timestamp that was last used to update this component */
	UFUNCTION(BlueprintPure, Category = "Components|ARFaceMesh")
	float GetLastUpdateTimestamp() const;

	/** Starts LiveLink publishing of this face component's data so that it can be used by the animation system */
	UFUNCTION(BlueprintCallable, Category = "Components|ARFaceMesh")
	void PublishViaLiveLink(FName SubjectName);

	/** Get the transform that the AR camera has detected */
	UFUNCTION(BlueprintPure, Category = "Components|ARFaceMesh")
	FTransform GetTransform() const;

	/**	Indicates whether the face mesh data should be built for rendering or not */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Components|ARFaceMesh")
	bool bWantsMeshUpdates;

	/**	Indicates whether collision should be created for this face mesh. This adds significant cost, so only use if you need to trace against the face mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Components|ARFaceMesh")
	bool bWantsCollision;

	/**	If true, the mesh data will come from the local ARKit face mesh data. The face mesh will update every tick and will handle loss of face tracking */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Components|ARFaceMesh")
	bool bAutoBindToLocalFaceMesh;

	/**	Determines how the transform from tracking data and the component's transform are mixed together */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Components|ARFaceMesh")
	EARFaceComponentTransformMixing TransformSetting;

	/**	If true, the face mesh will be rotated to face out of the screen (-X) rather than into the screen (+X) and corresponding axises to match */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Components|ARFaceMesh")
	bool bFlipTrackedRotation;

	/** Used when rendering the face mesh (mostly debug reasons) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Components|ARFaceMesh")
	TObjectPtr<class UMaterialInterface> FaceMaterial;

	/** Used to identify this component's face ar data uniquely as part of the LiveLink animation pipeline */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Components|ARFaceMesh")
	FName LiveLinkSubjectName;

	/** The set of changed curves to replicate to the other clients */
	UPROPERTY(ReplicatedUsing=OnRep_RemoteCurves, Transient)
	TArray<FNetQuantizeFaceCurve> RemoteCurves;

private:
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	virtual void InitializeComponent() override;
	virtual FMatrix GetRenderMatrix() const override;
	virtual class UMaterialInterface* GetMaterial(int32 ElementIndex) const override;
	virtual void SetMaterial(int32 ElementIndex, class UMaterialInterface* Material) override;

	/** Merges in the face curve deltas and pushes them to LiveLink */
	UFUNCTION()
	void OnRep_RemoteCurves();

	/**
	 * Sends the updated curves from the client to the server so that it can replicate to other clients
	 *
	 * @param ClientCurves the client's set of updated curves
	 */
	UFUNCTION(Reliable, Server, WithValidation)
	void ServerUpdateFaceCurves(const TArray<FNetQuantizeFaceCurve>& ClientCurves);

	/**
	 * Builds the delta set of curves needed for replication
	 *
	 * @param NewCurves the updated set of curves from the ar system
	 */
	void BuildUpdatedCurves(const FARBlendShapeMap& NewCurves);

	UARFaceGeometry* FindFaceGeometry();
	
	/** The current set of blend shapes for this component instance */
	FARBlendShapeMap BlendShapes;
	/** Transform of the face mesh */
	FTransform LocalToWorldTransform;

	/** The frame number this component was last updated on */
	uint32 LastUpdateFrameNumber;
	/** The time reported by the AR system that this object was last updated */
	double LastUpdateTimestamp;
	/** If this mesh is being published via LiveLink, the source to update with blendshapes */
	TSharedPtr<ILiveLinkSourceARKit> LiveLinkSource;
};
