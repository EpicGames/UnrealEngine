// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARSystem.h"
#include "ARTrackable.h"
#include "ILiveLinkSource.h"

#include "AppleARKitLiveLinkSourceFactory.generated.h"

struct FAppleARKitLiveLinkConnectionSettings;
class UTimecodeProvider;

/** Interface that publishes face ar blend shape information */
class APPLEARKITFACESUPPORT_API IARKitBlendShapePublisher
{
public:
	virtual void SetTimecodeProvider(UTimecodeProvider* InTimecodeProvider = nullptr) {}
	virtual void PublishBlendShapes(FName SubjectName, const FQualifiedFrameTime& FrameTime, const FARBlendShapeMap& BlendShapes, FName DeviceID = NAME_None) = 0;
	virtual void Shutdown() = 0;
};

/** Interface that publishes face ar blend shape information via LiveLink */
class APPLEARKITFACESUPPORT_API ILiveLinkSourceARKit :
	public IARKitBlendShapePublisher,
	public ILiveLinkSource
{
};

/** Factory that creates and registers the sources with the LiveLink client */
class APPLEARKITFACESUPPORT_API FAppleARKitLiveLinkSourceFactory
{
public:
	/** Creates a face mesh source that will autobind to the tracked face mesh */
	static TSharedPtr<ILiveLinkSourceARKit> CreateLiveLinkSource();

	/** Creates a AppleARKit source that holds a livelink remote listener. */
	static TSharedPtr<ILiveLinkSourceARKit> CreateLiveLinkSource(const FAppleARKitLiveLinkConnectionSettings& ConnectionSettings);

	/** Creates the publisher that will send remote events to a specified IP */
	static TSharedPtr<IARKitBlendShapePublisher, ESPMode::ThreadSafe> CreateLiveLinkRemotePublisher(const FString& RemoteAddr = FString());

	/** Creates the publisher that will write the curve data to disk */
	static TSharedPtr<IARKitBlendShapePublisher, ESPMode::ThreadSafe> CreateLiveLinkLocalFileWriter();
};

UCLASS()
class APPLEARKITFACESUPPORT_API UAppleARKitLiveLinkSourceFactory : public ULiveLinkSourceFactory
{
public:
	GENERATED_BODY()

	virtual FText GetSourceDisplayName() const override;
	virtual FText GetSourceTooltip() const override;

	virtual EMenuType GetMenuType() const override { return EMenuType::SubPanel; }
	virtual TSharedPtr<SWidget> BuildCreationPanel(FOnLiveLinkSourceCreated OnLiveLinkSourceCreated) const override;
	virtual TSharedPtr<ILiveLinkSource> CreateSource(const FString& ConnectionString) const override;

private:
	void CreateSourceFromSettings(const FAppleARKitLiveLinkConnectionSettings& ConnectionSettings, FOnLiveLinkSourceCreated OnSourceCreated) const;
};